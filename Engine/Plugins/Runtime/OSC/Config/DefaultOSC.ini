[CoreRedirects]
+FunctionRedirects=(OldName="OSCServer.SetWhitelistClientsEnabled",NewName="SetAllowlistClientsEnabled")
+FunctionRedirects=(OldName="OSCServer.AddWhitelistedClient",NewName="AddAllowlistedClient")
+FunctionRedirects=(OldName="OSCServer.RemoveWhitelistedClient",NewName="RemoveAllowlistedClient")
+FunctionRedirects=(OldName="OSCServer.ClearWhitelistedClients",NewName="ClearAllowlistedClients")
+FunctionRedirects=(OldName="OSCServer.GetWhitelistedClients",NewName="GetAllowlistedClients")