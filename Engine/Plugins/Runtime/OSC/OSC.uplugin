{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "2.0",
	"FriendlyName" : "OSC (Open Sound Control)",
	"Description" : "Implements the OSC 1.0 specification, allowing users to send and receive OSC messages and bundles between remote clients or applications.",
	"Category" : "Input Devices",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "OSC",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}