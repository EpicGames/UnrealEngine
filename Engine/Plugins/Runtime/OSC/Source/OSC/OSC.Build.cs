// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OSC : ModuleRules
	{
		public OSC(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"InputCore",
					"Networking",
					"Sockets",
				}
			);
		}
	}
}
