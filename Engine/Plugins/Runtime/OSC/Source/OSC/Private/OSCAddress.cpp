// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCAddress.h"

#include "Audio/AudioAddressPattern.h"
#include "OSCLog.h"


namespace UE::OSC
{
	const FString BundleTag = TEXT("#bundle");
	const FString PathSeparator = TEXT("/");
} // namespace UE::OSC


FOSCAddress::FOSCAddress()
	: bIsValidPattern(false)
	, bIsValidPath(false)
	, Hash(GetTypeHash(GetFullPath()))
{
}

FOSCAddress::FOSCAddress(const FString& InValue)
	: bIsValidPattern(false)
	, bIsValidPath(false)
{
	InValue.ParseIntoArray(Containers, *UE::OSC::PathSeparator, true);
	if (Containers.Num() > 0)
	{
		Method = Containers.Pop();
	}

	CacheAggregates();
}

void FOSCAddress::CacheAggregates()
{
	const bool bInvalidateSeparator = false;
	const FString FullPath = GetFullPath();

	Hash = GetTypeHash(FullPath);
	bIsValidPath = FAudioAddressPattern::IsValidPath(FullPath, bInvalidateSeparator);
	bIsValidPattern = FAudioAddressPattern::IsValidPattern(Containers, Method);
}

bool FOSCAddress::Matches(const FOSCAddress& InAddress) const
{
	if (IsValidPattern() && InAddress.IsValidPath())
	{
		return FAudioAddressPattern::PartsMatch(GetFullPath(), InAddress.GetFullPath());
	}

	return false;
}

bool FOSCAddress::IsValidPattern() const
{
	return bIsValidPattern;
}

bool FOSCAddress::IsValidPath() const
{
	return bIsValidPath;
}

bool FOSCAddress::PushContainer(FString Container)
{
	return PushContainers({ MoveTemp(Container) });
}

bool FOSCAddress::PushContainers(TArray<FString> NewContainers)
{
	if (NewContainers.IsEmpty())
	{
		return false;
	}

	for (const FString& Container : NewContainers)
	{
		if (Container.Contains(UE::OSC::PathSeparator))
		{
			UE_LOGF(LogOSC, Warning, "Failed to push containers on OSCAddress. "
				"Cannot contain OSC path separator '%ls'.", *UE::OSC::PathSeparator);
			return false;
		}
	}

	Containers.Append(MoveTemp(NewContainers));
	CacheAggregates();
	return true;
}

FString FOSCAddress::PopContainer()
{
	FString Popped;
	PopContainer(Popped);
	return Popped;
}

bool FOSCAddress::PopContainer(FString& OutContainer)
{
	if (Containers.IsEmpty())
	{
		OutContainer = { };
		return false;
	}

	OutContainer = Containers.Pop(EAllowShrinking::No);
	CacheAggregates();
	return true;
}

TArray<FString> FOSCAddress::PopContainers(int32 NumToPop)
{
	TArray<FString> Popped;
	PopContainers(NumToPop, Popped);
	return Popped;
}

bool FOSCAddress::PopContainers(int32 NumToPop, TArray<FString>& OutContainers)
{
	OutContainers = { };

	if (NumToPop <= 0 || Containers.IsEmpty())
	{
		return false;
	}

	while (!Containers.IsEmpty() && OutContainers.Num() != NumToPop)
	{
		OutContainers.Add(Containers.Pop(EAllowShrinking::No));
	}

	CacheAggregates();
	return true;
}

bool FOSCAddress::RemoveContainers(int32 InIndex, int32 InCount)
{
	if (InIndex >= 0 && InCount > 0)
	{
		if (InIndex + InCount <= Containers.Num())
		{
			Containers.RemoveAt(InIndex, InCount);
			CacheAggregates();
			return true;
		}
	}

	return false;
}

void FOSCAddress::ClearContainers()
{
	Containers.Reset();
	CacheAggregates();
}

const FString& FOSCAddress::GetMethod() const
{
	return Method;
}

bool FOSCAddress::GetNumericPrefix(int32& OutPreflix) const
{
	OutPreflix = 0;
	int32 Dec = 1;
	const TArray<TCHAR>& CharArray = Method.GetCharArray();
	for (
		int32 Index = 0;
		Index < CharArray.Num() && FChar::IsDigit(CharArray[Index]);
		++Index)
	{
		OutPreflix = (OutPreflix * Dec) + FChar::ConvertCharDigitToInt(CharArray[Index]);
		Dec *= 10;
	}

	return Dec == 1;
}

bool FOSCAddress::GetNumericSuffix(int32& OutSuffix) const
{
	OutSuffix = INDEX_NONE;
	int32 Dec = 1;
	const TArray<TCHAR>& CharArray = Method.GetCharArray();
	for (
		int32 Index = CharArray.Num() - 2 /* less term char */;
		Index >= 0 && FChar::IsDigit(CharArray[Index]);
		--Index)
	{
		OutSuffix += FChar::ConvertCharDigitToInt(CharArray[Index]) * Dec;
		Dec *= 10;
	}

	return Dec == 1;
}

bool FOSCAddress::Set(TArray<FString> NewContainers, FString NewMethod)
{
	Containers = { };

	if (PushContainers(MoveTemp(NewContainers)))
	{
		if (SetMethod(MoveTemp(NewMethod))) // Calls aggregate internally, so no need to call again on success
		{
			return true;
		}
	}

	Method = { };
	CacheAggregates();
	return false;
}

bool FOSCAddress::SetMethod(FString NewMethod)
{
	if (NewMethod.IsEmpty())
	{
		UE_LOGF(LogOSC, Warning, "Failed to set OSCAddress method. "
			"'InMethod' cannot be empty string.");
		return false;
	}

	if (NewMethod.Contains(UE::OSC::PathSeparator))
	{
		UE_LOGF(LogOSC, Warning, "Failed to set OSCAddress method. "
			"Cannot contain OSC path separator '%ls'.", *UE::OSC::PathSeparator);
		return false;
	}

	Method = MoveTemp(NewMethod);
	CacheAggregates();
	return true;
}

FString FOSCAddress::GetContainerPath() const
{
	return UE::OSC::PathSeparator + FString::Join(Containers, *UE::OSC::PathSeparator);
}

FString FOSCAddress::GetContainer(int32 Index) const
{
	if (Index >= 0 && Index < Containers.Num())
	{
		return Containers[Index];
	}

	return FString();
}

void FOSCAddress::GetContainers(TArray<FString>& OutContainers) const
{
	OutContainers = Containers;
}

FString FOSCAddress::GetFullPath() const
{
	if (Containers.Num() == 0)
	{
		return UE::OSC::PathSeparator + Method;
	}

	return GetContainerPath() + UE::OSC::PathSeparator + Method;
}
