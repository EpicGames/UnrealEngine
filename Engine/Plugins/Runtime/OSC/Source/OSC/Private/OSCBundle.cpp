// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCBundle.h"

#include "OSCBundlePacket.h"


FOSCBundle::FOSCBundle()
	: Packet(MakeShared<UE::OSC::FBundlePacket>())
{
}

FOSCBundle::FOSCBundle(const TSharedPtr<UE::OSC::IPacket>& InPacket)
	: Packet(InPacket.Get())
{
}

FOSCBundle::FOSCBundle(const TSharedRef<UE::OSC::IPacket>& InPacket)
	: Packet(InPacket)
{
}

void FOSCBundle::SetPacket(TSharedPtr<UE::OSC::IPacket>& InPacket)
{
	using namespace UE::OSC;

	check(InPacket->IsBundle());
	Packet = TSharedRef<IPacket>(InPacket.Get());
}

void FOSCBundle::SetPacket(const TSharedRef<UE::OSC::IPacket>& InPacket)
{
	check(InPacket->IsBundle());
	Packet = InPacket;
}

const TSharedPtr<UE::OSC::IPacket>& FOSCBundle::GetPacket() const
{
	static TSharedPtr<UE::OSC::IPacket> PacketPtr;
	PacketPtr = TSharedPtr<UE::OSC::IPacket>(&Packet.Get());
	return PacketPtr;
}

const TSharedRef<UE::OSC::IPacket>& FOSCBundle::GetPacketRef() const
{
	return Packet;
}
