// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCBundlePacket.h"

#include "OSCAddress.h"
#include "OSCLog.h"
#include "OSCStream.h"


namespace UE::OSC
{
	FBundlePacket::FBundlePacket(FIPv4Endpoint InEndpoint)
		: FPacketBase(InEndpoint)
		, TimeTag(0)
	{
	}

	void FBundlePacket::SetTimeTag(uint64 NewTimeTag)
	{
		TimeTag = FOSCData(NewTimeTag);
	}

	uint64 FBundlePacket::GetTimeTag() const
	{
		return TimeTag.GetTimeTag();
	}

	void FBundlePacket::WriteData(FStream& Stream)
	{
		// Write bundle & time tag
		Stream.WriteString(OSC::BundleTag);
		Stream.WriteUInt64(GetTimeTag());

		for (const TSharedRef<UE::OSC::IPacket>& Packet : Packets)
		{
			int32 StreamPos = Stream.GetPosition();
			Stream.WriteInt32(0);

			int32 InitPos = Stream.GetPosition();
			Packet->WriteData(Stream);
			int32 NewPos = Stream.GetPosition();

			Stream.SetPosition(StreamPos);
			Stream.WriteInt32(NewPos - InitPos);
			Stream.SetPosition(NewPos);
		}
	}

	void FBundlePacket::ReadData(FStream& Stream)
	{
		Packets.Reset();

		const FString ThisBundleTag = Stream.ReadString();
		if (ThisBundleTag != UE::OSC::BundleTag)
		{
			UE_LOGF(LogOSC, Warning, "Failed to parse OSCBundle of invalid format. #bundle identifier not first item in packet.");
			return;
		}

		TimeTag = FOSCData(Stream.ReadUInt64());

		while (!Stream.HasReachedEnd())
		{
			int32 PacketLength = Stream.ReadInt32();

			int32 StartPos = Stream.GetPosition();
			TSharedPtr<UE::OSC::IPacket> Packet = UE::OSC::IPacket::CreatePacket(Stream.GetData() + Stream.GetPosition(), IPEndpoint);
			if (!Packet.IsValid())
			{
				break;
			}

			Packet->ReadData(Stream);
			Packets.Add(Packet->AsShared());
			int32 EndPos = Stream.GetPosition();

			if (EndPos - StartPos != PacketLength)
			{
				UE_LOGF(LogOSC, Warning, "Failed to parse OSCBundle of invalid format. Element size mismatch.");
				break;
			}
		}
	}

	TArray<TSharedRef<UE::OSC::IPacket>>& FBundlePacket::GetPackets()
	{
		return Packets;
	}

	bool FBundlePacket::IsBundle()
	{
		return true;
	}

	bool FBundlePacket::IsMessage()
	{
		return false;
	}
} // namespace UE::OSC
