// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OSCMessagePacket.h"
#include "OSCPacket.h"
#include "OSCTypes.h"


namespace UE::OSC
{
	class FBundlePacket : public FPacketBase
	{
	public:
		FBundlePacket(FIPv4Endpoint InEndpoint = FIPv4Endpoint::Any);

		virtual ~FBundlePacket() = default;

		/** Set the bundle time tag. */
		void SetTimeTag(uint64 NewTimeTag);

		/** Get the bundle time tag. */
		uint64 GetTimeTag() const;

		/** Get OSC packet bundle. */
		TArray<TSharedRef<UE::OSC::IPacket>>& GetPackets();

		virtual bool IsBundle() override;
		virtual bool IsMessage() override;

		/** Writes bundle data into the OSC stream. */
		virtual void WriteData(FStream& Stream) override;

		/** Reads bundle data from provided OSC stream,
		  * adding packet data to internal packet bundle. */
		virtual void ReadData(FStream& Stream) override;

	private:
		/** Array of OSC packets. */
		TArray<TSharedRef<UE::OSC::IPacket>> Packets;

		/** Bundle time tag. */
		FOSCData TimeTag;
	};
} // namespace UE::OSC
