// Copyright Epic Games, Inc. All Rights Reserved.

#include "OSCClient.h"
#include "OSCClientProxy.h"


namespace UE::OSC
{
	TUniquePtr<IClientProxy> IClientProxy::Create(const FString& ClientName)
	{
		return MakeUnique<UE::OSC::FClientProxy>(ClientName);
	}
} // namespace UE::OSC

UOSCClient::UOSCClient(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ClientProxy(nullptr)
{
}

void UOSCClient::Connect()
{
	check(!ClientProxy.IsValid());
	ClientProxy = UE::OSC::IClientProxy::Create(GetName());
}

bool UOSCClient::IsActive() const
{
	return ClientProxy.IsValid() && ClientProxy->IsActive();
}

void UOSCClient::GetSendIPAddress(FString& InIPAddress, int32& Port)
{
	check(ClientProxy.IsValid());

	const FIPv4Endpoint Endpoint = ClientProxy->GetSendIPEndpoint();
	InIPAddress = Endpoint.Address.ToString();
	Port = Endpoint.Port;
}

bool UOSCClient::SetSendIPAddress(const FString& InIPAddress, const int32 Port)
{
	check(ClientProxy.IsValid());

	FIPv4Endpoint Endpoint;
	if (!FIPv4Address::Parse(InIPAddress, Endpoint.Address))
	{
		UE_LOGF(LogOSC, Warning, "Failed to set send IP Address: Could not parse IPAddress from string '%ls'.", *InIPAddress);
		return false;
	}
	Endpoint.Port = Port;

	ClientProxy->SetSendIPEndpoint(Endpoint);
	return true;
}

void UOSCClient::Stop()
{
	if (ClientProxy.IsValid())
	{
		ClientProxy->Stop();
	}
}

void UOSCClient::BeginDestroy()
{
	Stop();
	Super::BeginDestroy();
}

void UOSCClient::SendOSCMessage(FOSCMessage& Message)
{
	check(ClientProxy.IsValid());
	ClientProxy->SendMessage(Message);
}

void UOSCClient::SendOSCBundle(FOSCBundle& Bundle)
{
	check(ClientProxy.IsValid());
	ClientProxy->SendBundle(Bundle);
}
