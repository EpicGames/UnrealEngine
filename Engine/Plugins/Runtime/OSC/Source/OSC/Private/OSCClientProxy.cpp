// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCClientProxy.h"

#include "Common/UdpSocketBuilder.h"

#include "OSCBundle.h"
#include "OSCLog.h"
#include "OSCMessage.h"
#include "OSCMessagePacket.h"
#include "OSCStream.h"

namespace UE::OSC
{
	static const int32 OUTPUT_BUFFER_SIZE = 1024;

	FClientProxy::FClientProxy(const FString& InClientName)
		: Socket(FUdpSocketBuilder(InClientName).Build())
	{
	}

	FClientProxy::~FClientProxy()
	{
		if (Socket)
		{
			ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
			Socket = nullptr;
		}
	}

	const FIPv4Endpoint& FClientProxy::GetSendIPEndpoint() const
	{
		return IPEndpoint;
	}

	void FClientProxy::SetSendIPEndpoint(const FIPv4Endpoint& InEndpoint)
	{
		IPEndpoint = InEndpoint;
	}

	bool FClientProxy::IsActive() const
	{
		return Socket && Socket->GetConnectionState() == ESocketConnectionState::SCS_Connected;
	}

	void FClientProxy::SendPacket(UE::OSC::IPacket& Packet)
	{
		if (!Socket)
		{
#if !UE_BUILD_SHIPPING
			UE_LOGF(LogOSC, Error, "OSCClient stopped (socket '%ls') has been stopped. Failed to send msg", *DestroyedSocketDesc);
#endif // !UE_BUILD_SHIPPING

			return;
		}

		const FOSCAddress* OSCAddress = nullptr;
		if (Packet.IsMessage())
		{
			const FMessagePacket& MessagePacket = static_cast<FMessagePacket&>(Packet);
			OSCAddress = &MessagePacket.GetAddress();
			if (!OSCAddress->IsValidPath())
			{
				UE_LOGF(LogOSC, Warning, "Failed to write packet data. Invalid OSCAddress '%ls'", *OSCAddress->GetFullPath());
				return;
			}
		}

		if (IPEndpoint != FIPv4Endpoint::Any)
		{
			TSharedRef<FInternetAddr> InternetAddr = IPEndpoint.ToInternetAddr();
			FStream Stream = FStream();
			Packet.WriteData(Stream);
			const uint8* DataPtr = Stream.GetData();

			int32 BytesSent = 0;

			const int32 AttemptedLength = Stream.GetPosition();
			int32 Length = AttemptedLength;
			while (Length > 0)
			{
				const bool bSuccess = Socket->SendTo(DataPtr, Length, BytesSent, *InternetAddr);
				if (!bSuccess || BytesSent <= 0)
				{
					UE_LOGF(LogOSC, Verbose, "OSC Packet failed: Client '%ls', OSC Address '%ls', Send IP Endpoint %ls, Attempted Bytes = %d",
						*Socket->GetDescription(), OSCAddress ? *OSCAddress->GetFullPath() : *UE::OSC::BundleTag, *IPEndpoint.ToString(), AttemptedLength);
					return;
				}

				Length -= BytesSent;
				DataPtr += BytesSent;
			}

			UE_LOGF(LogOSC, Verbose, "OSC Packet sent: Client '%ls', OSC Address '%ls', Send IP Endpoint %ls, Bytes Sent = %d",
				*Socket->GetDescription(), OSCAddress ? *OSCAddress->GetFullPath() : *UE::OSC::BundleTag, *IPEndpoint.ToString(), AttemptedLength);
		}
	}

	void FClientProxy::SendMessage(const FOSCMessage& Message)
	{
		const TSharedRef<UE::OSC::IPacket>& Packet = Message.GetPacketRef();
		SendPacket(Packet.Get());
	}

	void FClientProxy::SendBundle(const FOSCBundle& Bundle)
	{
		const TSharedRef<UE::OSC::IPacket>& Packet = Bundle.GetPacketRef();
		SendPacket(Packet.Get());
	}

	void FClientProxy::Stop()
	{
		if (Socket)
		{
#if !UE_BUILD_SHIPPING
			DestroyedSocketDesc = Socket->GetDescription();
#endif // !UE_BUILD_SHIPPING
			ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);
			Socket = nullptr;
		}
	}
} // namespace UE::OSC
