// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OSCClient.h"


// Forward Declarations
class FInternetAddr;
class FSocket;


namespace UE::OSC
{
	// Forward Declarations
	class IPacket;

	class FClientProxy : public IClientProxy
	{
	public:
		FClientProxy(const FString& InClientName);
		virtual ~FClientProxy();

		UE_DEPRECATED(5.5, "Use GetSendIPEndpoint instead")
		virtual void GetSendIPAddress(FString& InIPAddress, int32& Port) const { }

		UE_DEPRECATED(5.5, "Use SetSendIPEndpoint instead")
		virtual bool SetSendIPAddress(const FString& InIPAddress, const int32 Port) { return false; }

		virtual const FIPv4Endpoint& GetSendIPEndpoint() const override;
		virtual void SetSendIPEndpoint(const FIPv4Endpoint& InEndpoint) override;

		bool IsActive() const override;

		void SendMessage(const FOSCMessage& Message) override;
		void SendBundle(const FOSCBundle& Bundle) override;

		void Stop() override;

		void SendPacket(UE::OSC::IPacket& Packet);

	private:
		/** Socket used to send the OSC packets. */
		FSocket* Socket = nullptr;

		/** IP Address used by socket. */
		FIPv4Endpoint IPEndpoint;

#if !UE_BUILD_SHIPPING
		FString DestroyedSocketDesc;
#endif // !UE_BUILD_SHIPPING
	};
} // namespace UE::OSC
