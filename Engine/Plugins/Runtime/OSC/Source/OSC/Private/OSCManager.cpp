// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCManager.h"

#include "HAL/IConsoleManager.h"
#include "OSCMessage.h"
#include "OSCMessagePacket.h"
#include "OSCBundle.h"
#include "OSCBundlePacket.h"
#include "OSCLog.h"
#include "OSCServer.h"
#include "OSCClient.h"

#include "Engine/World.h"
#include "Misc/Paths.h"
#include "SocketSubsystem.h"
#include "Trace/Trace.inl"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"


namespace UE::OSC
{
	namespace ManagerPrivate
	{
		static const int32 DefaultClientPort = 8094;
		static const int32 DefaultServerPort = 8095;

		// Returns true if provided address was null and was able to
		// override with local host address, false if not.
		bool GetLocalHostAddress(FString& InAddress)
		{
			if (!InAddress.IsEmpty() && InAddress != TEXT("0"))
			{
				return false;
			}

			bool bCanBind = false;
			bool bAppendPort = false;
			if (ISocketSubsystem* SocketSys = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
			{
				const TSharedPtr<FInternetAddr> Addr = SocketSys->GetLocalHostAddr(*GLog, bCanBind);
				if (Addr.IsValid())
				{
					InAddress = Addr->ToString(bAppendPort);
					return true;
				}
			}

			return false;
		}

		const FOSCData* GetDataAtIndex(const FOSCMessage& InMessage, const int32 InIndex)
		{
			const TSharedRef<FMessagePacket>& Packet = StaticCastSharedRef<FMessagePacket>(InMessage.GetPacketRef());
			const TArray<FOSCData>& Args = Packet->GetArguments();
			if (Args.IsValidIndex(InIndex))
			{
				return &Args[InIndex];
			}

			UE_LOGF(LogOSC, Warning, "Index '%d' out-of-bounds.  Message argument size = '%d'", InIndex, Args.Num());
			return nullptr;
		}

		void IterateMessageArgs(const FOSCMessage& InMessage, TFunctionRef<void(const FOSCData&)> InFunc)
		{
			const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(InMessage.GetPacketRef());
			const TArray<FOSCData>& Args = MessagePacket->GetArguments();
			for (int32 i = 0; i < Args.Num(); i++)
			{
				const FOSCData& OSCData = Args[i];
				InFunc(OSCData);
			}
		}

		void LogInvalidTypeAtIndex(EDataType DataType, int32 Index, const FOSCMessage& Msg)
		{
			UE_LOGF(LogOSC, Warning, "OSC Message Parse Failed: OSCData not %ls: index '%i', OSCAddress '%ls'",
				LexToString(DataType),
				Index,
				*Msg.GetAddress().GetFullPath());
		}

		static FAutoConsoleCommand GOSCPrintServers(
			TEXT("osc.servers"),
			TEXT("Prints diagnostic information pertaining to the currently initialized OSC servers objects to the output log."),
			FConsoleCommandDelegate::CreateStatic(
				[]()
				{
					FString LocalAddr;
					ManagerPrivate::GetLocalHostAddress(LocalAddr);
					UE_LOGF(LogOSC, Display, "Local IP: %ls", *LocalAddr);

					UE_LOGF(LogOSC, Display, "OSC Servers:");
					for (TObjectIterator<UOSCServer> Itr; Itr; ++Itr)
					{
						if (UOSCServer* Server = *Itr)
						{
							FString ToPrint = TEXT("    ") + Server->GetName();
							ToPrint.Appendf(TEXT(" (Id: %u"), Server->GetUniqueID());
							if (UWorld* World = Server->GetWorld())
							{
								ToPrint.Appendf(TEXT(", World: %s"), *World->GetName());
							}

							ToPrint.Appendf(TEXT(", IP: %s)"), *Server->GetIpAddress(true /* bIncludePort */));
							ToPrint += Server->IsActive() ? TEXT(" [Active]") : TEXT(" [Inactive]");

							UE_LOGF(LogOSC, Display, "%ls", *ToPrint);

							const TArray<FOSCAddress> BoundPatterns = Server->GetBoundOSCAddressPatterns();
							if (BoundPatterns.Num() > 0)
							{
								UE_LOGF(LogOSC, Display, "    Bound Address Patterns:");
								for (const FOSCAddress& Pattern : BoundPatterns)
								{
									UE_LOGF(LogOSC, Display, "         %ls", *Pattern.GetFullPath());
								}
								UE_LOGF(LogOSC, Display, "");
							}
						}
					}
				}
			)
		);

		static FAutoConsoleCommand GOSCServerConnect(
			TEXT("osc.server.connect"),
			TEXT("Connects or reconnects the osc mix server with the provided name\n"
				"(see \"osc.servers\" for a list of available servers and their respective names). Args:\n"
				"Name - Object name of server to (re)connect\n"
				"Address - IP Address to connect to (default: LocalHost)\n"
				"Port - Port to connect to (default: 8095)"),
			FConsoleCommandWithArgsDelegate::CreateStatic(
				[](const TArray<FString>& Args)
				{
					FString SrvName;
					if (Args.Num() > 0)
					{
						SrvName = Args[0];
					}

					FString IPAddr;
					GetLocalHostAddress(IPAddr);
					if (Args.Num() > 1)
					{
						IPAddr = Args[1];
					}

					int32 Port = DefaultServerPort;
					if (Args.Num() > 2)
					{
						Port = FCString::Atoi(*Args[2]);
					}

					for (TObjectIterator<UOSCServer> Itr; Itr; ++Itr)
					{
						if (UOSCServer* Server = *Itr)
						{
							if (Server->GetName() == SrvName)
							{
								Server->Stop();
								if (Server->SetAddress(IPAddr, Port))
								{
									Server->Listen();
								}
								return;
							}
						}
					}

					UE_LOGF(LogOSC, Warning, "Server object with name '%ls' not found, (re)connect not performed.", *SrvName);
				}
			)
		);

		static FAutoConsoleCommand GOSCServerConnectById(
			TEXT("osc.server.connectById"),
			TEXT("Connects or reconnects the osc mix server with the provided object id\n"
				"(see \"osc.servers\" for a list of available servers and their respective ids). Args:\n"
				"Id - Object Id of client to (re)connect\n"
				"Address - IP Address to (re)connect to (default: LocalHost)\n"
				"Port - Port to (re)connect to (default: 8095)"),
			FConsoleCommandWithArgsDelegate::CreateStatic(
				[](const TArray<FString>& Args)
				{
					if (Args.Num() == 0)
					{
						return;
					}

					const int32 SrvId = FCString::Atoi(*Args[0]);

					FString IPAddr;
					GetLocalHostAddress(IPAddr);
					if (Args.Num() > 1)
					{
						IPAddr = Args[1];
					}

					int32 Port = DefaultServerPort;
					if (Args.Num() > 2)
					{
						Port = FCString::Atoi(*Args[2]);
					}

					for (TObjectIterator<UOSCServer> Itr; Itr; ++Itr)
					{
						if (UOSCServer* Server = *Itr)
						{
							if (Server->GetUniqueID() == SrvId)
							{
								Server->Stop();
								if (Server->SetAddress(IPAddr, Port))
								{
									Server->Listen();
								}
								return;
							}
						}
					}

					UE_LOGF(LogOSC, Warning, "Server object with id '%u' not found, (re)connect not performed.", SrvId);
				}
			)
		);

		static FAutoConsoleCommand GOSCPrintClients(
			TEXT("osc.clients"),
			TEXT("Prints diagnostic information pertaining to the currently initialized OSC client objects to the output log."),
			FConsoleCommandDelegate::CreateStatic(
				[]()
				{
					FString LocalAddr;
					GetLocalHostAddress(LocalAddr);
					UE_LOGF(LogOSC, Display, "Local IP: %ls", *LocalAddr);

					UE_LOGF(LogOSC, Display, "OSC Clients:");
					for (TObjectIterator<UOSCClient> Itr; Itr; ++Itr)
					{
						if (UOSCClient* Client = *Itr)
						{
							FString ToPrint = TEXT("    ") + Client->GetName();
							ToPrint.Appendf(TEXT(" (Id: %u"), Client->GetUniqueID());
							if (UWorld* World = Client->GetWorld())
							{
								ToPrint.Appendf(TEXT(", World: %s"), *World->GetName());
							}

							FString IPAddrStr;
							int32 Port;
							Client->GetSendIPAddress(IPAddrStr, Port);
							ToPrint += TEXT(", Send IP: ") + IPAddrStr + TEXT(":");
							ToPrint.AppendInt(Port);
							ToPrint += Client->IsActive() ? TEXT(") [Active]") : TEXT(") [Inactive]");

							UE_LOGF(LogOSC, Display, "%ls", *ToPrint);
						}
					}
				}
			)
		);

		static FAutoConsoleCommand GOSCClientConnect(
			TEXT("osc.client.connect"),
			TEXT("Connects (or reconnects) the osc mix client with the provided name\n"
				"(see \"osc.clients\" for a list of available clients and their respective ids). Args:\n"
				"Name - Object name of client to (re)connect\n"
				"Address - IP Address to (re)connect to (default: LocalHost)\n"
				"Port - Port to (re)connect to (default: 8094)"),
			FConsoleCommandWithArgsDelegate::CreateStatic(
				[](const TArray< FString >& Args)
				{
					if (Args.Num() == 0)
					{
						return;
					}
					const FString CliName = Args[0];

					FString IPAddr;
					GetLocalHostAddress(IPAddr);
					if (Args.Num() > 1)
					{
						IPAddr = Args[1];
					}

					int32 Port = DefaultClientPort;
					if (Args.Num() > 2)
					{
						Port = FCString::Atoi(*Args[2]);
					}

					for (TObjectIterator<UOSCClient> Itr; Itr; ++Itr)
					{
						if (UOSCClient* Client = *Itr)
						{
							if (Client->GetName() == CliName)
							{
								Client->Connect();
								Client->SetSendIPAddress(IPAddr, Port);
								return;
							}
						}
					}

					UE_LOGF(LogOSC, Warning, "Client object with name '%ls' not found, (re)connect not performed.", *CliName);
				}
			)
		);

		static FAutoConsoleCommand GOSCClientConnectById(
			TEXT("osc.client.connectById"),
			TEXT("Connects (or reconnects) the osc mix client with the provided object id\n"
				"(see \"osc.clients\" for a list of available clients and their respective ids). Args:\n"
				"Id - Object Id of client to (re)connect\n"
				"Address - IP Address to (re)connect to (default: LocalHost)\n"
				"Port - Port to (re)connect to (default: 8094)"),
			FConsoleCommandWithArgsDelegate::CreateStatic(
				[](const TArray< FString >& Args)
				{
					int32 CliId = INDEX_NONE;
					if (Args.Num() > 0)
					{
						CliId = FCString::Atoi(*Args[0]);
					}

					FString IPAddr;
					GetLocalHostAddress(IPAddr);
					if (Args.Num() > 1)
					{
						IPAddr = Args[1];
					}

					int32 Port = DefaultClientPort;
					if (Args.Num() > 2)
					{
						Port = FCString::Atoi(*Args[2]);
					}

					for (TObjectIterator<UOSCClient> Itr; Itr; ++Itr)
					{
						if (UOSCClient* Client = *Itr)
						{
							if (Client->GetUniqueID() == CliId)
							{
								Client->Connect();
								Client->SetSendIPAddress(IPAddr, Port);
								return;
							}
						}
					}

					UE_LOGF(LogOSC, Warning, "Client object with id '%u' not found, (re)connect not performed.", CliId);
				}
			)
		);
	} // namespace ManagerPrivate

	int32 GetDefaultClientPort()
	{
		return ManagerPrivate::DefaultClientPort;
	}

	int32 GetDefaultServerPort()
	{
		return ManagerPrivate::DefaultServerPort;
	}
} // namespace UE::OSC

UOSCServer* UOSCManager::CreateOSCServer(FString InReceiveIPAddress, int32 InPort, bool bInMulticastLoopback, bool bInStartListening, FString ServerName, UObject* Outer)
{
	using namespace UE::OSC;

	if (ManagerPrivate::GetLocalHostAddress(InReceiveIPAddress))
	{
		UE_LOGF(LogOSC, Display, "OSCServer ReceiveAddress not specified. Using LocalHost IP: '%ls'", *InReceiveIPAddress);
	}

	if (ServerName.IsEmpty())
	{
		ServerName = FString::Printf(TEXT("OSCServer_%s"), *FGuid::NewGuid().ToString(EGuidFormats::Short));
	}

	UOSCServer* NewOSCServer = nullptr;
	if (Outer)
	{
		NewOSCServer = NewObject<UOSCServer>(Outer, *ServerName, RF_StrongRefOnFrame);
	}
	else
	{
		UE_LOGF(LogOSC, Warning, "Outer object not set.  OSCServer may be garbage collected if not referenced.");
		NewOSCServer = NewObject<UOSCServer>(GetTransientPackage(), *ServerName);
	}

	if (NewOSCServer)
	{
		NewOSCServer->SetMulticastLoopback(bInMulticastLoopback);
		if (NewOSCServer->SetAddress(InReceiveIPAddress, InPort))
		{
			if (bInStartListening)
			{
				NewOSCServer->Listen();
			}
		}
		else
		{
			UE_LOGF(LogOSC, Warning, "Failed to parse ReceiveAddress '%ls' for OSCServer.", *InReceiveIPAddress);
		}

		return NewOSCServer;
	}

	return nullptr;
}

UOSCClient* UOSCManager::CreateOSCClient(FString InSendIPAddress, int32 InPort, FString ClientName, UObject* Outer)
{
	using namespace UE::OSC;

	if (ManagerPrivate::GetLocalHostAddress(InSendIPAddress))
	{
		UE_LOGF(LogOSC, Display, "OSCClient SendAddress not specified. Using LocalHost IP: '%ls'", *InSendIPAddress);
	}

	if (ClientName.IsEmpty())
	{
		ClientName = FString::Printf(TEXT("OSCClient_%s"), *FGuid::NewGuid().ToString(EGuidFormats::Short));
	}

	UOSCClient* NewOSCClient = nullptr;
	if (Outer)
	{
		NewOSCClient = NewObject<UOSCClient>(Outer, *ClientName, RF_StrongRefOnFrame);
	}
	else
	{
		UE_LOGF(LogOSC, Warning, "Outer object not set.  OSCClient '%ls' may be garbage collected if not referenced.", *ClientName);
		NewOSCClient = NewObject<UOSCClient>(GetTransientPackage(), *ClientName);
	}

	if (NewOSCClient)
	{
		NewOSCClient->Connect();
		if (!NewOSCClient->SetSendIPAddress(InSendIPAddress, InPort))
		{
			UE_LOGF(LogOSC, Warning, "Failed to parse SendAddress '%ls' for OSCClient. Client unable to send new messages.", *InSendIPAddress);
		}

		return NewOSCClient;
	}

	return nullptr;
}

FOSCMessage& UOSCManager::ClearMessage(FOSCMessage& OutMessage)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& Packet = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	Packet->EmptyArguments();

	return OutMessage;
}

FOSCBundle& UOSCManager::ClearBundle(FOSCBundle& OutBundle)
{
	using namespace UE::OSC;

	const TSharedRef<FBundlePacket>& Packet = StaticCastSharedRef<FBundlePacket>(OutBundle.GetPacketRef());
	Packet->GetPackets().Reset();

	return OutBundle;
}

FOSCBundle& UOSCManager::AddMessageToBundle(const FOSCMessage& InMessage, FOSCBundle& Bundle)
{
	using namespace UE::OSC;

	const TSharedRef<FBundlePacket>& BundlePacket = StaticCastSharedRef<FBundlePacket>(Bundle.GetPacketRef());
	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(InMessage.GetPacketRef());

	BundlePacket->GetPackets().Add(MessagePacket);

	return Bundle;
}

FOSCBundle& UOSCManager::AddBundleToBundle(const FOSCBundle& InBundle, FOSCBundle& OutBundle)
{
	using namespace UE::OSC;

	const TSharedRef<FBundlePacket>& InBundlePacket = StaticCastSharedRef<FBundlePacket>(InBundle.GetPacketRef());
	const TSharedRef<FBundlePacket>& OutBundlePacket = StaticCastSharedRef<FBundlePacket>(OutBundle.GetPacketRef());

	InBundlePacket->GetPackets().Add(OutBundlePacket);

	return OutBundle;
}

FOSCMessage& UOSCManager::AddFloat(FOSCMessage& OutMessage, float InValue)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	MessagePacket->AddArgument(FOSCData(InValue));
	return OutMessage;
}

FOSCMessage& UOSCManager::AddInt32(FOSCMessage& OutMessage, int32 InValue)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	MessagePacket->AddArgument(FOSCData(InValue));
	return OutMessage;
}

FOSCMessage& UOSCManager::AddInt64(FOSCMessage& OutMessage, int64 InValue)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	MessagePacket->AddArgument(FOSCData(InValue));
	return OutMessage;
}

FOSCMessage& UOSCManager::AddAddress(FOSCMessage& OutMessage, const FOSCAddress& InValue)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	MessagePacket->AddArgument(FOSCData(InValue.GetFullPath()));
	return OutMessage;
}

FOSCMessage& UOSCManager::AddString(FOSCMessage& OutMessage, FString InValue)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	MessagePacket->AddArgument(FOSCData(InValue));
	return OutMessage;
}

FOSCMessage& UOSCManager::AddBlob(FOSCMessage& OutMessage, const TArray<uint8>& OutValue)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	MessagePacket->AddArgument(FOSCData(OutValue));
	return OutMessage;
}

FOSCMessage& UOSCManager::AddBool(FOSCMessage& OutMessage, bool InValue)
{
	using namespace UE::OSC;

	const TSharedRef<FMessagePacket>& MessagePacket = StaticCastSharedRef<FMessagePacket>(OutMessage.GetPacketRef());
	MessagePacket->AddArgument(FOSCData(InValue));
	return OutMessage;
}

TArray<FOSCBundle> UOSCManager::GetBundlesFromBundle(const FOSCBundle& InBundle)
{
	using namespace UE::OSC;

	TArray<FOSCBundle> Bundles;
	const TSharedRef<FBundlePacket>& BundlePacket = StaticCastSharedRef<FBundlePacket>(InBundle.GetPacketRef());
	for (int32 i = 0; i < BundlePacket->GetPackets().Num(); i++)
	{
		const TSharedRef<UE::OSC::IPacket>& Packet = BundlePacket->GetPackets()[i];
		if (Packet->IsBundle())
		{
			Bundles.Add(FOSCBundle(Packet));
		}
	}

	return Bundles;
}

FOSCMessage UOSCManager::GetMessageFromBundle(const FOSCBundle& InBundle, int32 InIndex, bool& bOutSucceeded)
{
	using namespace UE::OSC;

	const TSharedRef<FBundlePacket>& BundlePacket = StaticCastSharedRef<FBundlePacket>(InBundle.GetPacketRef());
	int32 Count = 0;
	for (const TSharedRef<UE::OSC::IPacket>& Packet : BundlePacket->GetPackets())
	{
		if (Packet->IsMessage())
		{
			if (InIndex == Count)
			{
				bOutSucceeded = true;
				return FOSCMessage(Packet);
			}
			Count++;
		}
	}

	bOutSucceeded = false;
	return FOSCMessage();
}

TArray<FOSCMessage> UOSCManager::GetMessagesFromBundle(const FOSCBundle& InBundle)
{
	using namespace UE::OSC;

	TArray<FOSCMessage> Messages;
	const TSharedRef<FBundlePacket>& BundlePacket = StaticCastSharedRef<FBundlePacket>(InBundle.GetPacketRef());
	for (int32 i = 0; i < BundlePacket->GetPackets().Num(); i++)
	{
		const TSharedRef<UE::OSC::IPacket>& Packet = BundlePacket->GetPackets()[i];
		if (Packet->IsMessage())
		{
			Messages.Add(FOSCMessage(Packet));
		}
	}
	
	return Messages;
}

bool UOSCManager::GetAddress(const FOSCMessage& InMessage, const int32 InIndex, FOSCAddress& OutValue)
{
	using namespace UE::OSC;

	if (const FOSCData* OSCData = ManagerPrivate::GetDataAtIndex(InMessage, InIndex))
	{
		if (OSCData->IsString())
		{
			OutValue = FOSCAddress(OSCData->GetString());
			return OutValue.IsValidPath();
		}
		ManagerPrivate::LogInvalidTypeAtIndex(EDataType::String, InIndex, InMessage);
	}

	OutValue = FOSCAddress();
	return false;
}

void UOSCManager::GetAllAddresses(const FOSCMessage& InMessage, TArray<FOSCAddress>& OutValues)
{
	using namespace UE::OSC;

	ManagerPrivate::IterateMessageArgs(InMessage, [&](const FOSCData& OSCData)
	{
		if (OSCData.IsString())
		{
			FOSCAddress AddressToAdd = FOSCAddress(OSCData.GetString());
			if (AddressToAdd.IsValidPath())
			{
				OutValues.Add(MoveTemp(AddressToAdd));
			}
		}
	});
}

bool UOSCManager::GetFloat(const FOSCMessage& InMessage, const int32 InIndex, float& OutValue)
{
	using namespace UE::OSC;

	OutValue = 0.0f;
	if (const FOSCData* OSCData = ManagerPrivate::GetDataAtIndex(InMessage, InIndex))
	{
		if (OSCData->IsFloat())
		{
			OutValue = OSCData->GetFloat();
			return true;
		}
		ManagerPrivate::LogInvalidTypeAtIndex(EDataType::Float, InIndex, InMessage);
	}

	return false;
}

void UOSCManager::GetAllFloats(const FOSCMessage& InMessage, TArray<float>& OutValues)
{
	using namespace UE::OSC;

	ManagerPrivate::IterateMessageArgs(InMessage, [&](const FOSCData& OSCData)
	{
		if (OSCData.IsFloat())
		{
			OutValues.Add(OSCData.GetFloat());
		}
	});
}

bool UOSCManager::GetInt32(const FOSCMessage& InMessage, const int32 InIndex, int32& OutValue)
{
	using namespace UE::OSC;

	OutValue = 0;
	if (const FOSCData* OSCData = ManagerPrivate::GetDataAtIndex(InMessage, InIndex))
	{
		if (OSCData->IsInt32())
		{
			OutValue = OSCData->GetInt32();
			return true;
		}
		ManagerPrivate::LogInvalidTypeAtIndex(EDataType::Int32, InIndex, InMessage);
	}

	return false;
}

void UOSCManager::GetAllInt32s(const FOSCMessage& InMessage, TArray<int32>& OutValues)
{
	using namespace UE::OSC;

	ManagerPrivate::IterateMessageArgs(InMessage, [&](const FOSCData& OSCData)
	{
		if (OSCData.IsInt32())
		{
			OutValues.Add(OSCData.GetInt32());
		}
	});
}

bool UOSCManager::GetInt64(const FOSCMessage& InMessage, const int32 InIndex, int64& OutValue)
{
	using namespace UE::OSC;

	OutValue = 0l;
	if (const FOSCData* OSCData = ManagerPrivate::GetDataAtIndex(InMessage, InIndex))
	{
		if (OSCData->IsInt64())
		{
			OutValue = OSCData->GetInt64();
			return true;
		}
		ManagerPrivate::LogInvalidTypeAtIndex(EDataType::Int64, InIndex, InMessage);
	}

	return false;
}

void UOSCManager::GetAllInt64s(const FOSCMessage& InMessage, TArray<int64>& OutValues)
{
	using namespace UE::OSC;

	ManagerPrivate::IterateMessageArgs(InMessage, [&](const FOSCData& OSCData)
	{
		if (OSCData.IsInt64())
		{
			OutValues.Add(OSCData.GetInt64());
		}
	});
}

bool UOSCManager::GetString(const FOSCMessage& InMessage, const int32 InIndex, FString& OutValue)
{
	using namespace UE::OSC;

	if (const FOSCData* OSCData = ManagerPrivate::GetDataAtIndex(InMessage, InIndex))
	{
		if (OSCData->IsString())
		{
			OutValue = OSCData->GetString();
			return true;
		}
		ManagerPrivate::LogInvalidTypeAtIndex(EDataType::String, InIndex, InMessage);
	}

	OutValue.Reset();
	return false;
}

void UOSCManager::GetAllStrings(const FOSCMessage& InMessage, TArray<FString>& OutValues)
{
	using namespace UE::OSC;

	ManagerPrivate::IterateMessageArgs(InMessage, [&](const FOSCData& OSCData)
	{
		if (OSCData.IsString())
		{
			OutValues.Add(OSCData.GetString());
		}
	});
}

bool UOSCManager::GetBool(const FOSCMessage& InMessage, const int32 InIndex, bool& OutValue)
{
	using namespace UE::OSC;

	OutValue = false;
	if (const FOSCData* OSCData = ManagerPrivate::GetDataAtIndex(InMessage, InIndex))
	{
		if (OSCData->IsBool())
		{
			OutValue = OSCData->GetBool();
			return true;
		}
		ManagerPrivate::LogInvalidTypeAtIndex(EDataType::True, InIndex, InMessage);
	}

	return false;
}

void UOSCManager::GetAllBools(const FOSCMessage& InMessage, TArray<bool>& OutValues)
{
	using namespace UE::OSC;

	ManagerPrivate::IterateMessageArgs(InMessage, [&](const FOSCData& OSCData)
	{
		if (OSCData.IsBool())
		{
			OutValues.Add(OSCData.GetBool());
		}
	});
}

bool UOSCManager::GetBlob(const FOSCMessage& InMessage, const int32 InIndex, TArray<uint8>& OutValue)
{
	using namespace UE::OSC;

	OutValue.Reset();
	if (const FOSCData* OSCData = ManagerPrivate::GetDataAtIndex(InMessage, InIndex))
	{
		if (OSCData->IsBlob())
		{
			OutValue = OSCData->GetBlob();
			return true;
		}
		ManagerPrivate::LogInvalidTypeAtIndex(EDataType::Blob, InIndex, InMessage);
	}

	return false;
}

bool UOSCManager::OSCAddressIsValidPath(const FOSCAddress& InAddress)
{
	return InAddress.IsValidPath();
}

bool UOSCManager::OSCAddressIsValidPattern(const FOSCAddress& InAddress)
{
	return InAddress.IsValidPattern();
}

FOSCAddress UOSCManager::ConvertStringToOSCAddress(const FString& InString)
{
	return FOSCAddress(InString);
}

UObject* UOSCManager::FindObjectAtOSCAddress(const FOSCAddress& InAddress)
{
	FSoftObjectPath Path(ObjectPathFromOSCAddress(InAddress));
	if (Path.IsValid())
	{
		return Path.TryLoad();
	}

	UE_LOGF(LogOSC, Verbose, "Failed to load object from OSCAddress '%ls'", *InAddress.GetFullPath());
	return nullptr;
}

FOSCAddress UOSCManager::OSCAddressFromObjectPath(UObject* InObject)
{
	const FString Path = FPaths::ChangeExtension(InObject->GetPathName(), FString());
	return FOSCAddress(Path);
}

FOSCAddress UOSCManager::OSCAddressFromObjectPathString(const FString& InPathName)
{
	TArray<FString> PartArray;
	InPathName.ParseIntoArray(PartArray, TEXT("\'"));

	// Type declaration at beginning of path. Assumed to be in the form <SomeTypeContainer1'/Container2/ObjectName.ObjectName'>
	if (PartArray.Num() > 1)
	{
		const FString NoExtPath = FPaths::SetExtension(PartArray[1], TEXT(""));
		return FOSCAddress(NoExtPath);
	}

	// No type declaration at beginning of path. Assumed to be in the form <Container1/Container2/ObjectName.ObjectName>
	if (PartArray.Num() > 0)
	{
		const FString NoExtPath = FPaths::SetExtension(PartArray[0], TEXT(""));
		return FOSCAddress(NoExtPath);
	}

	// Invalid address
	return FOSCAddress();
}

FString UOSCManager::ObjectPathFromOSCAddress(const FOSCAddress& InAddress)
{
	const FString Path = InAddress.GetFullPath() + TEXT(".") + InAddress.GetMethod();
	return Path;
}

FOSCAddress& UOSCManager::OSCAddressPushContainer(FOSCAddress& OutAddress, const FString& InToAppend)
{
	OutAddress.PushContainer(InToAppend);
	return OutAddress;
}

FOSCAddress& UOSCManager::OSCAddressPushContainers(FOSCAddress& OutAddress, const TArray<FString>& InToAppend)
{
	OutAddress.PushContainers(InToAppend);
	return OutAddress;
}

FString UOSCManager::OSCAddressPopContainer(FOSCAddress& OutAddress)
{
	return OutAddress.PopContainer();
}

TArray<FString> UOSCManager::OSCAddressPopContainers(FOSCAddress& OutAddress, int32 InNumContainers)
{
	return OutAddress.PopContainers(InNumContainers);
}

FOSCAddress& UOSCManager::OSCAddressRemoveContainers(FOSCAddress& OutAddress, int32 InIndex, int32 InCount)
{
	OutAddress.RemoveContainers(InIndex, InCount);
	return OutAddress;
}

bool UOSCManager::OSCAddressPathMatchesPattern(const FOSCAddress& InPattern, const FOSCAddress& InPath)
{
	return InPattern.Matches(InPath);
}

FOSCAddress UOSCManager::GetOSCMessageAddress(const FOSCMessage& InMessage)
{
	return InMessage.GetAddress();
}

FOSCMessage& UOSCManager::SetOSCMessageAddress(FOSCMessage& OutMessage, const FOSCAddress& InAddress)
{
	OutMessage.SetAddress(InAddress);
	return OutMessage;
}

FString UOSCManager::GetOSCAddressContainer(const FOSCAddress& InAddress, int32 InIndex)
{
	return InAddress.GetContainer(InIndex);
}

TArray<FString> UOSCManager::GetOSCAddressContainers(const FOSCAddress& InAddress)
{
	TArray<FString> Containers;
	InAddress.GetContainers(Containers);
	return Containers;
}

FString UOSCManager::GetOSCAddressContainerPath(const FOSCAddress& InAddress)
{
	return InAddress.GetContainerPath();
}

FString UOSCManager::GetOSCAddressFullPath(const FOSCAddress& InAddress)
{
	return InAddress.GetFullPath();
}

FString UOSCManager::GetOSCAddressMethod(const FOSCAddress& InAddress)
{
	return InAddress.GetMethod();
}

FOSCAddress& UOSCManager::ClearOSCAddressContainers(FOSCAddress& OutAddress)
{
	OutAddress.ClearContainers();
	return OutAddress;
}

FOSCAddress& UOSCManager::SetOSCAddressMethod(FOSCAddress& OutAddress, const FString& InMethod)
{
	OutAddress.SetMethod(InMethod);
	return OutAddress;
}
