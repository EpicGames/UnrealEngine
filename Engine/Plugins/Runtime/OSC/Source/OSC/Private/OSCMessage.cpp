// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCMessage.h"

#include "OSCLog.h"
#include "OSCMessagePacket.h"


FOSCMessage::FOSCMessage()
	: Packet(MakeShared<UE::OSC::FMessagePacket>())
{
}

FOSCMessage::FOSCMessage(FOSCAddress Address, TArray<UE::OSC::FOSCData> Args)
	: Packet(MakeShared<UE::OSC::FMessagePacket>())
{
	using namespace UE::OSC;
	TSharedRef<FMessagePacket> MsgPacket = StaticCastSharedRef<FMessagePacket>(Packet);
	MsgPacket->SetAddress(MoveTemp(Address));
	MsgPacket->SetArguments(MoveTemp(Args));
}

FOSCMessage::FOSCMessage(const TSharedRef<UE::OSC::IPacket>& InPacket)
	: Packet(InPacket)
{
}

FOSCMessage::FOSCMessage(const TSharedPtr<UE::OSC::IPacket>& InPacket)
	: FOSCMessage::FOSCMessage()
{
	Packet = InPacket.ToSharedRef();
}

void FOSCMessage::SetPacket(TSharedPtr<UE::OSC::IPacket>& InPacket)
{
	Packet = InPacket.ToSharedRef();
}

void FOSCMessage::SetPacket(TSharedRef<UE::OSC::IPacket>& InPacket)
{
	Packet = InPacket;
}

const TSharedPtr<UE::OSC::IPacket>& FOSCMessage::GetPacket() const
{
	static TSharedPtr<UE::OSC::IPacket> RetPacketPtr;
	RetPacketPtr = TSharedPtr<UE::OSC::IPacket>(&Packet.Get());
	return RetPacketPtr;
}

const TSharedRef<UE::OSC::IPacket>& FOSCMessage::GetPacketRef() const
{
	return Packet;
}

const TArray<UE::OSC::FOSCData>& FOSCMessage::GetArgumentsChecked() const
{
	using namespace UE::OSC;
	return StaticCastSharedRef<FMessagePacket>(Packet)->GetArguments();
}

bool FOSCMessage::SetAddress(const FOSCAddress& InAddress)
{
	using namespace UE::OSC;

	if (!InAddress.IsValidPath())
	{
		UE_LOGF(LogOSC, Warning, "Attempting to set invalid OSCAddress '%ls'. OSC address must begin with '/'", *InAddress.GetFullPath());
		return false;
	}

	StaticCastSharedRef<FMessagePacket>(Packet)->SetAddress(InAddress);
	return true;
}

const FOSCAddress& FOSCMessage::GetAddress() const
{
	using namespace UE::OSC;

	return StaticCastSharedRef<FMessagePacket>(Packet)->GetAddress();
}
