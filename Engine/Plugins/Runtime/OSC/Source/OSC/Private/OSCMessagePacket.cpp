// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCMessagePacket.h"

#include "OSCLog.h"
#include "OSCStream.h"
#include "OSCTypes.h"


namespace UE::OSC
{
	FPacketBase::FPacketBase(FIPv4Endpoint InEndpoint)
		: IPEndpoint(InEndpoint)
	{
	}

	const FIPv4Endpoint& FPacketBase::GetIPEndpoint() const
	{
		return IPEndpoint;
	}

	FMessagePacket::FMessagePacket(const FIPv4Endpoint& InEndpoint)
		: FPacketBase(InEndpoint)
	{
	}

	void FMessagePacket::AddArgument(FOSCData OSCData)
	{
		Arguments.Add(MoveTemp(OSCData));
	}

	void FMessagePacket::EmptyArguments()
	{
		Arguments.Empty();
	}

	const FOSCAddress& FMessagePacket::GetAddress() const
	{
		return Address;
	}

	const TArray<FOSCData>& FMessagePacket::GetArguments() const
	{
		return Arguments;
	}

	void FMessagePacket::SetAddress(FOSCAddress InAddress)
	{
		Address = MoveTemp(InAddress);
	}

	void FMessagePacket::SetArguments(TArray<FOSCData> OSCData)
	{
		Arguments = MoveTemp(OSCData);
	}

	bool FMessagePacket::IsBundle()
	{
		return false;
	}

	bool FMessagePacket::IsMessage()
	{
		return true;
	}

	void FMessagePacket::WriteData(FStream& Stream)
	{
		if (!Address.IsValidPath())
		{
			UE_LOGF(LogOSC, Warning, "Failed to write OSCMessagePacket. Invalid OSCAddress '%ls'", *Address.GetFullPath());
			return;
		}

		// Begin writing data 
		Stream.WriteString(Address.GetFullPath());

		// write type tags
		FString TagTypes = ",";
		for (int32 i = 0; i < Arguments.Num(); i++)
		{
			TagTypes += static_cast<ANSICHAR>(Arguments[i].GetDataType());
		}

		Stream.WriteString(TagTypes);

		for (const FOSCData& OSCData : Arguments)
		{
			switch (OSCData.GetDataType())
			{
			case EDataType::Char:
				Stream.WriteChar(OSCData.GetChar());
				break;
			case EDataType::Int32:
				Stream.WriteInt32(OSCData.GetInt32());
				break;
			case EDataType::Float:
				Stream.WriteFloat(OSCData.GetFloat());
				break;
			case EDataType::Double:
				Stream.WriteDouble(OSCData.GetDouble());
				break;
			case EDataType::Int64:
				Stream.WriteInt64(OSCData.GetInt64());
				break;
			case EDataType::Time:
				Stream.WriteUInt64(OSCData.GetTimeTag());
				break;
			case EDataType::String:
				Stream.WriteString(OSCData.GetString());
				break;
			case EDataType::Blob:
			{
				TArray<uint8> blob = OSCData.GetBlob();
				Stream.WriteBlob(blob);
			}
			break;
			case EDataType::Color:
	#if PLATFORM_LITTLE_ENDIAN
				Stream.WriteInt32(OSCData.GetColor().ToPackedABGR());
	#else
				Stream.WriteInt32(OSCData.GetColor().ToPackedRGBA());
	#endif
				break;
			case EDataType::True:
			case EDataType::False:
			case EDataType::NilValue:
			case EDataType::Infinitum:
				// No values are written for these types
				break;
			default:
				// Argument is not supported 
				unimplemented();
				break;
			}
		}
	}

	void FMessagePacket::ReadData(FStream& Stream)
	{
		// Read Address
		Address = Stream.ReadString();

		// Read string of tags
		const FString StreamString = Stream.ReadString();

		const TArray<TCHAR, FString::AllocatorType>& TagTypes = StreamString.GetCharArray();
		if(TagTypes.Num() == 0)
		{
			UE_LOGF(LogOSC, Error, "Failed to read message packet with address '%ls' from stream: Invalid (Empty) Type Tag", *Address.GetFullPath());
			return;
		}

		// Skip the first argument which is ','
		for (int32 i = 1; i < TagTypes.Num(); i++)
		{
			const EDataType Tag = static_cast<EDataType>(TagTypes[i]);
			switch (Tag)
			{
			case EDataType::Char:
				Arguments.Add(FOSCData(Stream.ReadChar()));
				break;
			case EDataType::Int32:
				Arguments.Add(FOSCData(Stream.ReadInt32()));
				break;
			case EDataType::Float:
				Arguments.Add(FOSCData(Stream.ReadFloat()));
				break;
			case EDataType::Double:
				Arguments.Add(FOSCData(Stream.ReadDouble()));
				break;
			case EDataType::Int64:
				Arguments.Add(FOSCData(Stream.ReadInt64()));
				break;
			case EDataType::True:
				Arguments.Add(FOSCData(true));
				break;
			case EDataType::False:
				Arguments.Add(FOSCData(false));
				break;
			case EDataType::NilValue:
				Arguments.Add(FOSCData::NilData());
				break;
			case EDataType::Infinitum:
				Arguments.Add(FOSCData::Infinitum());
				break;
			case EDataType::Time:
				Arguments.Add(FOSCData(Stream.ReadUInt64()));
				break;
			case EDataType::String:
				Arguments.Add(FOSCData(Stream.ReadString()));
				break;
			case EDataType::Blob:
				Arguments.Add(FOSCData(Stream.ReadBlob()));
				break;
			case EDataType::Color:
				Arguments.Add(FOSCData(FColor(Stream.ReadInt32())));
				break;
			case EDataType::Terminate:
				// Return on first terminate found. FString GetCharArray can return
				// an array with multiple terminators.
				return;

			default:
				// Argument is not supported 
				unimplemented();
				break;
			}
		}
	}
} // namespace UE::OSC
