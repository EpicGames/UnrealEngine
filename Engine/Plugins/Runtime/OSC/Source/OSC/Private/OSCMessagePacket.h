// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OSCAddress.h"
#include "OSCPacket.h"

class FOSCType;


namespace UE::OSC
{
	class FPacketBase : public IPacket
	{
	public:
		FPacketBase(FIPv4Endpoint InEndpoint = FIPv4Endpoint::Any);

		virtual ~FPacketBase() = default;

		/** Get endpoint IP address and port responsible for creation/forwarding of packet */
		virtual const FIPv4Endpoint& GetIPEndpoint() const override;

	protected:
		FIPv4Endpoint IPEndpoint;
	};


	class FMessagePacket : public FPacketBase
	{
	public:
		FMessagePacket() = default;
		FMessagePacket(const FIPv4Endpoint& InEndpoint);
		virtual ~FMessagePacket() = default;

		/** Adds argument to argument array */
		void AddArgument(FOSCData OSCData);

		/** Empties all arguments */
		void EmptyArguments();

		/** Set OSC message address. */
		void SetAddress(FOSCAddress InAddress);

		/** Sets argument array to the given values */
		void SetArguments(TArray<FOSCData> Args);

		/** Get OSC message address. */
		virtual const FOSCAddress& GetAddress() const;

		/** Get arguments array. */
		virtual const TArray<FOSCData>& GetArguments() const;

		/** Returns false to indicate type is not OSC bundle. */
		virtual bool IsBundle();

		/** Returns true to indicate its an OSC message. */
		virtual bool IsMessage();

		/** Write message data into an OSC stream. */
		virtual void WriteData(FStream& Stream) override;

		/** Reads message data from an OSC stream and creates new argument. */
		virtual void ReadData(FStream& Stream) override;

	private:
		/** OSC address. */
		FOSCAddress Address;

		/** List of argument data types. */
		TArray<FOSCData> Arguments;
	};
} // namespace UE::OSC
