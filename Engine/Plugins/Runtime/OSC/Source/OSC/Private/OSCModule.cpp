// Copyright Epic Games, Inc. All Rights Reserved.

#include "OSCModule.h"

#include "OSCLog.h"

DEFINE_LOG_CATEGORY(LogOSC);


namespace UE::OSC
{
	void FModule::StartupModule()
	{
		if (!FModuleManager::Get().LoadModule(TEXT("Networking")))
		{
			UE_LOGF(LogOSC, Error, "Required module 'Networking' failed to load. OSC service disabled.");
		}
	}
} // namespace UE::OSC

IMPLEMENT_MODULE(UE::OSC::FModule, OSC)
