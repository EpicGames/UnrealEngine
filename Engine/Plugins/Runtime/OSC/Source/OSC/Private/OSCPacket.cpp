// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCPacket.h"

#include "OSCMessagePacket.h"
#include "OSCBundlePacket.h"
#include "OSCLog.h"


namespace UE::OSC
{
	TSharedPtr<IPacket> IPacket::CreatePacket(const uint8* InPacketType, const FIPv4Endpoint& InIPEndpoint)
	{
		const FString PacketIdentifier(ANSI_TO_TCHAR((const ANSICHAR*)&InPacketType[0]));
	
		TSharedPtr<FPacketBase> Packet;
		if (PacketIdentifier.StartsWith(OSC::PathSeparator))
		{
			Packet = MakeShared<FMessagePacket>(InIPEndpoint);
		}
		else if (PacketIdentifier == UE::OSC::BundleTag)
		{
			Packet = MakeShared<OSC::FBundlePacket>(InIPEndpoint);
		}
		else
		{
			UE_LOGF(LogOSC, Warning, "Failed to parse lead character of OSC packet. "
				"Lead identifier of '%c' not valid bundle tag ('%ls') or message ('%ls') identifier.", PacketIdentifier[0], *OSC::BundleTag, *OSC::PathSeparator);
			return nullptr;
		}

		return Packet;
	}
} // namespace UE::OSC
