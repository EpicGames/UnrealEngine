// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCServer.h"

#include "Containers/Ticker.h"
#include "OSCAddress.h"
#include "OSCBundle.h"
#include "OSCBundlePacket.h"
#include "OSCLog.h"
#include "OSCMessage.h"
#include "OSCMessagePacket.h"
#include "OSCPacket.h"
#include "OSCServerProxy.h"


namespace UE::OSC
{
	TSharedPtr<IServerProxy> IServerProxy::Create()
	{
		return MakeShared<OSC::FServerProxy>();
	}

	bool IServerProxy::SetAddress(const FString& InReceiveIPAddress, int32 InPort)
	{
		FIPv4Endpoint Endpoint;
		FIPv4Address::Parse(InReceiveIPAddress, Endpoint.Address);
		Endpoint.Port = InPort;
		return SetIPEndpoint(Endpoint);
	}
} // namespace UE::OSC

UOSCServer::UOSCServer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UOSCServer::GetMulticastLoopback() const
{
	check(ServerProxy.IsValid());
	return ServerProxy->GetMulticastLoopback();
}

bool UOSCServer::IsActive() const
{
	check(ServerProxy.IsValid());
	return ServerProxy->IsActive();
}

void UOSCServer::Listen()
{
	check(ServerProxy.IsValid());

	ClearPacketsInternal();
	ServerProxy->Listen(GetName());

	TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float /*Time*/)
	{
		PumpPacketQueue();
		return true;
	}));
}

bool UOSCServer::SetAddress(const FString& InReceiveIPAddress, int32 InPort)
{
	check(ServerProxy.IsValid());

	FIPv4Address Address;
	FIPv4Address::Parse(InReceiveIPAddress, Address);
	return ServerProxy->SetIPEndpoint(FIPv4Endpoint(Address, InPort));
}

void UOSCServer::SetMulticastLoopback(bool bInMulticastLoopback)
{
	check(ServerProxy.IsValid());
	ServerProxy->SetMulticastLoopback(bInMulticastLoopback);
}

#if WITH_EDITOR
void UOSCServer::SetTickInEditor(bool bInTickInEditor)
{
}
#endif // WITH_EDITOR

void UOSCServer::Stop()
{
	FTSTicker::RemoveTicker(TickHandle);
	TickHandle.Reset();

	// CDO May have this not set by initializer ctor, so have to check if valid
	if (ServerProxy.IsValid())
	{
		ServerProxy->SetOnDispatchPacket({ });
		ServerProxy->Stop();
	}

	ClearPacketsInternal();
}

void UOSCServer::BeginDestroy()
{
	Stop();
	Super::BeginDestroy();
}

void UOSCServer::PostInitProperties()
{
	using namespace UE::OSC;

	Super::PostInitProperties();

	const UClass* ThisClass = UOSCServer::StaticClass();
	check(ThisClass);

	const UObject* DefaultObj = ThisClass->GetDefaultObject();
	check(DefaultObj);

	if (DefaultObj != this)
	{
		OSCPackets = MakeShared<FPacketQueue>();
		ServerProxy = IServerProxy::Create();
	}
}

void UOSCServer::SetAllowlistClientsEnabled(bool bEnabled)
{
	check(ServerProxy.IsValid());
	ServerProxy->SetFilterClientsByAllowList(bEnabled);
}

void UOSCServer::AddAllowlistedClient(const FString& InIPAddress, int32 IPPort)
{
	check(ServerProxy.IsValid());

	FIPv4Address NewAddress;
	if (FIPv4Address::Parse(InIPAddress, NewAddress))
	{
		ServerProxy->AddClientEndpointToAllowList(FIPv4Endpoint(NewAddress, IPPort));
	}
}

void UOSCServer::RemoveAllowlistedClient(const FString& InIPAddress, int32 IPPort)
{
	check(ServerProxy.IsValid());

	FIPv4Address NewAddress;
	if (FIPv4Address::Parse(InIPAddress, NewAddress))
	{
		ServerProxy->RemoveClientEndpointFromAllowList(FIPv4Endpoint(NewAddress, IPPort));
	}
}

void UOSCServer::ClearAllowlistedClients()
{
	check(ServerProxy.IsValid());
	ServerProxy->ClearClientEndpointAllowList();
}

FString UOSCServer::GetIpAddress(bool bIncludePort) const
{
	check(ServerProxy.IsValid());

	if (bIncludePort)
	{
		return ServerProxy->GetIPEndpoint().ToString();
	}

	return ServerProxy->GetIPEndpoint().Address.ToString();
}

int32 UOSCServer::GetPort() const
{
	check(ServerProxy.IsValid());
	return ServerProxy->GetIPEndpoint().Port;
}

TSet<FString> UOSCServer::GetAllowlistedClients(bool bIncludePort) const
{
	check(ServerProxy.IsValid());
	const TSet<FIPv4Endpoint>& Endpoints = ServerProxy->GetClientEndpointAllowList();

	TSet<FString> Result;
	Algo::Transform(Endpoints, Result, [&bIncludePort](const FIPv4Endpoint& ClientEndpoint)
	{
		return bIncludePort
			? ClientEndpoint.ToString()
			: ClientEndpoint.Address.ToString();
	});

	return Result;
}

void UOSCServer::BindEventToOnOSCAddressPatternMatchesPath(const FOSCAddress& InOSCAddressPattern, const FOSCDispatchMessageEventBP& InEvent)
{
	if (InOSCAddressPattern.IsValidPattern())
	{
		FOSCDispatchMessageEvent& MessageEvent = AddressPatterns.FindOrAdd(InOSCAddressPattern);
		MessageEvent.AddUnique(InEvent);
	}
}

void UOSCServer::UnbindEventFromOnOSCAddressPatternMatchesPath(const FOSCAddress& InOSCAddressPattern, const FOSCDispatchMessageEventBP& InEvent)
{
	if (InOSCAddressPattern.IsValidPattern())
	{
		if (FOSCDispatchMessageEvent* AddressPatternEvent = AddressPatterns.Find(InOSCAddressPattern))
		{
			AddressPatternEvent->Remove(InEvent);
			if (!AddressPatternEvent->IsBound())
			{
				AddressPatterns.Remove(InOSCAddressPattern);
			}
		}
	}
}

void UOSCServer::UnbindAllEventsFromOnOSCAddressPatternMatchesPath(const FOSCAddress& InOSCAddressPattern)
{
	if (InOSCAddressPattern.IsValidPattern())
	{
		AddressPatterns.Remove(InOSCAddressPattern);
	}
}

void UOSCServer::UnbindAllEventsFromOnOSCAddressPatternMatching()
{
	AddressPatterns.Reset();
}

TArray<FOSCAddress> UOSCServer::GetBoundOSCAddressPatterns() const
{
	TArray<FOSCAddress> OutAddressPatterns;
	AddressPatterns.GetKeys(OutAddressPatterns);
	return OutAddressPatterns;
}

void UOSCServer::ClearPacketsInternal()
{
	using namespace UE::OSC;

	OSCPackets = MakeShared<FPacketQueue>();

	if (ServerProxy.IsValid())
	{
		ServerProxy->SetOnDispatchPacket(MakeShared<IServerProxy::FOnDispatchPacket>([Queue = OSCPackets](TSharedRef<UE::OSC::IPacket> Packet)
		{
			Queue->Enqueue(Packet);
		}));
	}
}

void UOSCServer::BroadcastBundle(const FOSCBundle& InBundle)
{
	using namespace UE::OSC;

	const TSharedRef<IPacket>& Packet = InBundle.GetPacketRef();
	const FIPv4Endpoint& Endpoint = Packet->GetIPEndpoint();
	const FString AddrStr = Endpoint.Address.ToString();

	OnOscBundleReceived.Broadcast(InBundle, AddrStr, Endpoint.Port);
	OnOscBundleReceivedNative.Broadcast(InBundle, AddrStr, Endpoint.Port);

	TSharedRef<FBundlePacket> BundlePacket = StaticCastSharedRef<FBundlePacket>(InBundle.GetPacketRef());
	const TArray<TSharedRef<IPacket>>& Packets = BundlePacket->GetPackets();
	for (const TSharedRef<IPacket>& SubPacket : Packets)
	{
		if (SubPacket->IsMessage())
		{
			BroadcastMessage(FOSCMessage(SubPacket));
		}
		else if (SubPacket->IsBundle())
		{
			BroadcastBundle(FOSCBundle(SubPacket));
		}
		else
		{
			UE_LOGF(LogOSC, Warning, "Failed to parse invalid received message. Invalid OSC type (packet is neither identified as message nor bundle).");
		}
	}
}

void UOSCServer::BroadcastMessage(const FOSCMessage& InMessage)
{
	using namespace UE::OSC;

	const TSharedRef<IPacket>& Packet = InMessage.GetPacketRef();
	const FIPv4Endpoint& Endpoint = Packet->GetIPEndpoint();
	const FString AddrStr = Endpoint.Address.ToString();

	OnOscMessageReceived.Broadcast(InMessage, AddrStr, Endpoint.Port);
	OnOscMessageReceivedNative.Broadcast(InMessage, AddrStr, Endpoint.Port);

	UE_LOGF(LogOSC, Verbose, "Message received from IP endpoint '%ls', OSCAddress of '%ls'.", *Endpoint.ToString(), *InMessage.GetAddress().GetFullPath());

	for (const TPair<FOSCAddress, FOSCDispatchMessageEvent>& Pair : AddressPatterns)
	{
		const FOSCDispatchMessageEvent& DispatchEvent = Pair.Value;
		if (Pair.Key.Matches(InMessage.GetAddress()))
		{
			DispatchEvent.Broadcast(Pair.Key, InMessage, AddrStr, Endpoint.Port);
			UE_LOGF(LogOSC, Verbose, "Message dispatched from IP endpoint '%ls', OSCAddress path of '%ls' matched OSCAddress pattern '%ls'.",
				*Endpoint.ToString(),
				*InMessage.GetAddress().GetFullPath(),
				*Pair.Key.GetFullPath());
		}
	}
}

void UOSCServer::PumpPacketQueue()
{
	using namespace UE::OSC;

	check(IsInGameThread());

	TSharedPtr<IPacket> Packet;
	while (OSCPackets->Dequeue(Packet))
	{
		// Safe to cast to ref as all added items when dispatching to queue should be valid by this point
		TSharedRef<IPacket> PacketRef = Packet.ToSharedRef();
		if (ServerProxy->CanProcessPacket(PacketRef))
		{
			if (PacketRef->IsMessage())
			{
				BroadcastMessage(FOSCMessage(PacketRef));
			}
			else if (PacketRef->IsBundle())
			{
				BroadcastBundle(FOSCBundle(PacketRef));
			}
			else
			{
				UE_LOGF(LogOSC, Warning, "Failed to parse invalid received message. Invalid OSC type (packet is neither identified as message nor bundle).");
			}
		}
	}
}
