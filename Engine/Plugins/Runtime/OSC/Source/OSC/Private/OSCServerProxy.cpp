// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCServerProxy.h"

#include "Common/UdpSocketBuilder.h"

#include "OSCLog.h"
#include "OSCPacket.h"
#include "OSCStream.h"


namespace UE::OSC
{
	FServerProxy::FServerProxy()
	{
		ClientAllowList.Add(FIPv4Endpoint::Any);
	}

	FServerProxy::~FServerProxy()
	{
		Stop();
	}

	bool FServerProxy::CanProcessPacket(TSharedRef<UE::OSC::IPacket> Packet) const
	{
		// 1. Check if filtering anything
		if (ClientAllowList.Contains(FIPv4Endpoint::Any))
		{
			return true;
		}

		// 2. Check for explicit endpoint
		FIPv4Endpoint EndpointToTest = Packet->GetIPEndpoint();
		if (ClientAllowList.Contains(EndpointToTest))
		{
			return true;
		}

		// 2. Check for explicit address & wildcard 'any' port endpoint
		EndpointToTest.Port = FIPv4Endpoint::Any.Port;
		return ClientAllowList.Contains(EndpointToTest);
	}

	void FServerProxy::OnPacketReceived(FConstPacketDataRef InData, const FIPv4Endpoint& InEndpoint)
	{
		TSharedPtr<UE::OSC::IPacket> Packet = UE::OSC::IPacket::CreatePacket(InData->GetData(), InEndpoint);
		if (!Packet.IsValid())
		{
			UE_LOGF(LogOSC, Verbose, "Message received from endpoint '%ls' invalid OSC packet.", *InEndpoint.ToString());
			return;
		}

		FStream Stream = FStream(InData->GetData(), InData->Num());
		Packet->ReadData(Stream);

		FScopeLock Lock(&MutateDispatchFunctionCritSec);
		if (OnDispatchFunction.IsValid())
		{
			(*OnDispatchFunction)(Packet.ToSharedRef());
		}
	}

	FString FServerProxy::GetIpAddress() const
	{
		return Endpoint.Address.ToString();
	}

	int32 FServerProxy::GetPort() const
	{
		return Endpoint.Port;
	}

	const FIPv4Endpoint& FServerProxy::GetIPEndpoint() const
	{
		return Endpoint;
	}

	FString FServerProxy::GetDescription() const
	{
		if (ServerReceiver.IsValid())
		{
			return ServerReceiver->GetDescription();
		}

		return { };
	}

	bool FServerProxy::GetMulticastLoopback() const
	{
		return bMulticastLoopback;
	}

	bool FServerProxy::IsActive() const
	{
		return ServerReceiver.IsValid();
	}

	void FServerProxy::Listen(const FString& InServerName)
	{
		if (IsActive())
		{
			UE_LOGF(LogOSC, Error, "OSCServer currently '%ls' currently listening to endpoint '%ls'. Failed to start new service prior to calling stop.",
				*InServerName, *Endpoint.Address.ToString());
			return;
		}

		FServerReceiver::FOptions Options;
		Options.bMulticastLoopback = bMulticastLoopback;
		Options.ReceivedDataDelegate.BindSP(this, &FServerProxy::OnPacketReceived);

		ServerReceiver = FServerReceiver::Launch(InServerName, Endpoint, MoveTemp(Options));
	}

	bool FServerProxy::SetIPEndpoint(const FIPv4Endpoint& InEndpoint)
	{
		if (IsActive())
		{
			UE_LOGF(LogOSC, Error, "Cannot set '%ls' endpoint to '%ls' while OSC server is currently active.", *GetDescription(), *InEndpoint.ToString());
			return false;
		}

		Endpoint = InEndpoint;
		return true;
	}

	bool FServerProxy::SetMulticastLoopback(bool bInMulticastLoopback)
	{
		if (bInMulticastLoopback != bMulticastLoopback && IsActive())
		{
			UE_LOGF(LogOSC, Error, "Cannot update MulticastLoopback while OSCServer is active.");
			return false;
		}

		bMulticastLoopback = bInMulticastLoopback;
		return true;
	}

	void FServerProxy::SetOnDispatchPacket(TSharedPtr<FOnDispatchPacket> OnDispatch)
	{
		FScopeLock Lock(&MutateDispatchFunctionCritSec);
		OnDispatchFunction = OnDispatch;
	}

	void FServerProxy::Stop()
	{
		ServerReceiver.Reset();
	}

	void FServerProxy::AddClientToAllowList(const FString& InIPAddress)
	{
		FIPv4Endpoint EndpointToAdd;
		if (!FIPv4Address::Parse(InIPAddress, EndpointToAdd.Address))
		{
			UE_LOGF(LogOSC, Warning, "OSCServer failed to add IP Address '%ls' to allow list. Address is invalid.", *InIPAddress);
			return;
		}

		ClientAllowList.Add(EndpointToAdd);
	}

	void FServerProxy::RemoveClientFromAllowList(const FString& InIPAddress)
	{
		FIPv4Endpoint EndpointToRemove;
		if (!FIPv4Address::Parse(InIPAddress, EndpointToRemove.Address))
		{
			UE_LOGF(LogOSC, Warning, "OSCServer failed to remove IP Address '%ls' from allow list. Address is invalid.", *InIPAddress);
			return;
		}

		ClientAllowList.Remove(EndpointToRemove);
	}

	const TSet<FIPv4Endpoint>& FServerProxy::GetClientEndpointAllowList() const
	{
		return ClientAllowList;
	}

	void FServerProxy::AddClientEndpointToAllowList(const FIPv4Endpoint& InIPEndpoint)
	{
		ClientAllowList.Add(InIPEndpoint);
	}

	void FServerProxy::RemoveClientEndpointFromAllowList(const FIPv4Endpoint& InIPEndpoint)
	{
		ClientAllowList.Add(InIPEndpoint);
	}

	void FServerProxy::ClearClientEndpointAllowList()
	{
		ClientAllowList.Empty();
	}

	void FServerProxy::SetFilterClientsByAllowList(bool bInEnabled)
	{
		if (bInEnabled)
		{
			ClientAllowList.Remove(FIPv4Endpoint::Any);
		}
		else
		{
			ClientAllowList.Empty(1);
			ClientAllowList.Add(FIPv4Endpoint::Any);
		}
	}
} // namespace UE::OSC
