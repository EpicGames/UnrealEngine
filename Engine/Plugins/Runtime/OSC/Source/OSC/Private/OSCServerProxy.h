// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "HAL/CriticalSection.h"

#include "OSCServer.h"
#include "OSCServerReceiver.h"

struct FIPv4Endpoint;


namespace UE::OSC
{
	class FServerProxy : public IServerProxy , public TSharedFromThis<FServerProxy>
	{
	public:
		FServerProxy();
		virtual ~FServerProxy();

		// Begin IServerProxy interface
		virtual void AddClientToAllowList(const FString& InIPAddress) override;
		virtual void AddClientEndpointToAllowList(const FIPv4Endpoint& InIPv4Endpoint) override;

		virtual bool CanProcessPacket(TSharedRef<UE::OSC::IPacket> Packet) const override;
		virtual void ClearClientEndpointAllowList() override;

		virtual const TSet<FIPv4Endpoint>& GetClientEndpointAllowList() const override;
		virtual FString GetDescription() const override;
		virtual const FIPv4Endpoint& GetIPEndpoint() const override;
		virtual bool GetMulticastLoopback() const override;

		UE_DEPRECATED(5.5, "Use GetIPEndpoint instead")
		virtual FString GetIpAddress() const override;

		UE_DEPRECATED(5.5, "Use GetIPEndpoint instead")
		virtual int32 GetPort() const override;

		virtual bool IsActive() const override;

		virtual void Listen(const FString& InServerName) override;

		virtual bool SetIPEndpoint(const FIPv4Endpoint& InEndpoint) override;
		virtual bool SetMulticastLoopback(bool bInMulticastLoopback) override;
		virtual void SetOnDispatchPacket(TSharedPtr<FOnDispatchPacket> OnDispatch) override;
		virtual void Stop() override;

		virtual void RemoveClientFromAllowList(const FString& InIPAddress) override;
		virtual void RemoveClientEndpointFromAllowList(const FIPv4Endpoint& InIPv4Endpoint) override;

		virtual void SetFilterClientsByAllowList(bool bInEnabled) override;
		// End IServerProxy interface

	private:
		/** Callback that receives data from a socket. */
		void OnPacketReceived(FConstPacketDataRef InData, const FIPv4Endpoint& InEndpoint);

		mutable FCriticalSection MutateDispatchFunctionCritSec;

		TSharedPtr<FServerReceiver> ServerReceiver;

		/** Only packets from this list of client addresses will be processed if bFilterClientsByAllowList is true. */
		TSet<FIPv4Endpoint> ClientAllowList;

		/** Endpoint to listen for OSC packets on. If set to 'Any', defaults to LocalHost */
		FIPv4Endpoint Endpoint = FIPv4Endpoint::Any;

		/** Whether or not to loopback if address provided is multicast */
		bool bMulticastLoopback = false;

		TSharedPtr<FOnDispatchPacket> OnDispatchFunction;
	};
} // namespace UE::OSC
