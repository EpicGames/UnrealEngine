// Copyright Epic Games, Inc. All Rights Reserved.

#include "OSCServerReceiver.h"


namespace UE::OSC
{
	namespace ServerReceieverPrivate
	{
		ISocketSubsystem& GetSocketSubsystemChecked()
		{
			ISocketSubsystem* Subsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
			check(Subsystem);
			return *Subsystem;
		}

		void SetAsLocalHostIfLoopback(ISocketSubsystem& InSocketSubsystem, FIPv4Endpoint& OutEndpoint)
		{
			if (OutEndpoint.Address.IsLoopbackAddress())
			{
				check(GLog);

				bool bCanBind = false;
				TSharedRef<FInternetAddr> LocalIP = InSocketSubsystem.GetLocalHostAddr(*GLog, bCanBind);
				uint32 LocalIPAddr = 0;
				LocalIP->GetIp(LocalIPAddr);
				OutEndpoint.Address = FIPv4Address(LocalIPAddr);
			}
		}
	} // namespace ServerReceieverPrivate

	FServerReceiver::FServerReceiver(FPrivateToken, FOptions InOptions)
		: Options(MoveTemp(InOptions))
	{
	}

	FServerReceiver::~FServerReceiver()
	{
		bStopping = true;

		UE_LOGF(LogOSC, Display, "Destroying OSC Socket Receiver '%ls'", *GetDescription());
		if (Thread)
		{
			const FString& ThreadName = Thread->GetThreadName();
			UE_LOGF(LogOSC, Display, "Killing process thread '%ls'...", *ThreadName);
			constexpr bool bWait = true;
			Thread->Kill(bWait);
			UE_LOGF(LogOSC, Display, "Process thread '%ls' killed successfully.", *ThreadName);
			Thread.Reset();
		}

		if (Socket)
		{
			const FString SocketDesc = Socket->GetDescription();
			const bool bSocketClosed = Socket->Close();
			UE_LOGF(LogOSC, Display, "Socket '%ls' '%ls' closed.", *SocketDesc, bSocketClosed ? TEXT("successfully") : TEXT("failed to"));
			Socket.Reset();
		}
	}

	bool FServerReceiver::BindSocket(ISocketSubsystem& SocketSubsystem, const FIPv4Endpoint& InEndpoint, TSharedRef<FInternetAddr> RemoteAddr, const FString& InName)
	{
		check(Socket);

		if (!Socket->SetNonBlocking(true) ||
			!Socket->SetReuseAddr(false) ||
			!Socket->SetBroadcast(false) ||
			!Socket->SetRecvErr())
		{
			const ESocketErrors SocketError = SocketSubsystem.GetLastErrorCode();

			UE_LOGF(LogOSC, Warning, "OSC Socket Receiver: Failed to configure socket %ls: Error code %d: %ls",
				*InName, int32(SocketError), SocketSubsystem.GetSocketError(SocketError));

			return false;
		}

		if (!Socket->Bind(*RemoteAddr))
		{
			const ESocketErrors SocketError = SocketSubsystem.GetLastErrorCode();

			UE_LOGF(LogOSC, Warning, "OSC Socket Receiver: Failed to bind %ls to %ls. Error code %d: %ls",
				*InName, *InEndpoint.ToString(), int32(SocketError), SocketSubsystem.GetSocketError(SocketError));

			return false;
		}

		return true;
	}

	bool FServerReceiver::InitMulticast(ISocketSubsystem& InSocketSubsystem, const FIPv4Endpoint& InEndpoint, TSharedRef<FInternetAddr> RemoteAddr, const FString& InName)
	{
		if (!Socket->SetMulticastLoopback(Options.bMulticastLoopback) || !Socket->SetMulticastTtl(1))
		{
			const ESocketErrors SocketError = InSocketSubsystem.GetLastErrorCode();
			UE_LOGF(LogOSC, Warning, "OSC Socket Receiver: Failed to configure multicast for %ls (loopback: %i). Error code %d.",
				*InName, Options.bMulticastLoopback, int32(SocketError));
			return false;
		}

		TSharedRef<FInternetAddr> MulticastAddress = InSocketSubsystem.CreateInternetAddr(RemoteAddr->GetProtocolType());
		MulticastAddress->SetBroadcastAddress();

		TSharedPtr<FInternetAddr> AddressToUse;
		if (InEndpoint.Address.IsSessionFrontendMulticast() && MulticastAddress->GetProtocolType() != FNetworkProtocolTypes::IPv4)
		{
			AddressToUse = MulticastAddress;
		}
		else
		{
			AddressToUse = FIPv4Endpoint(InEndpoint.Address, 0).ToInternetAddr();
		}

		if (!Socket->JoinMulticastGroup(*AddressToUse, *FIPv4Endpoint(FIPv4Address::Any, 0).ToInternetAddr()))
		{
			const ESocketErrors SocketError = InSocketSubsystem.GetLastErrorCode();
			UE_LOGF(LogOSC, Warning, "OSC Server Receiver: Failed to subscribe %ls to multicast group %ls. Error code %d.",
				*InName, *InEndpoint.Address.ToString(), int32(SocketError));
			return false;
		}

		return true;
	}

	bool FServerReceiver::InitSocket(const FString& InName, FIPv4Endpoint Endpoint)
	{
		using namespace ServerReceieverPrivate;

		ISocketSubsystem& SocketSubsystem = GetSocketSubsystemChecked();
		SetAsLocalHostIfLoopback(SocketSubsystem, Endpoint);

		TSharedRef<FInternetAddr> RemoteAddr = Endpoint.ToInternetAddr();
		Socket = SocketSubsystem.CreateUniqueSocket(NAME_DGram, *InName, RemoteAddr->GetProtocolType());
		if (!Socket.IsValid())
		{
			UE_LOGF(LogOSC, Warning, "OSCServer '%ls' failed to create socket", *InName);
			return false;
		}

		const bool bSocketBound = BindSocket(SocketSubsystem, Endpoint, RemoteAddr, InName);
		if (!bSocketBound)
		{
			return false;
		}

		if (Endpoint.Address.IsMulticastAddress())
		{
			const bool bMulticastInit = InitMulticast(SocketSubsystem, Endpoint, RemoteAddr, InName);
			if (!bMulticastInit)
			{
				return false;
			}
		}

		return true;
	}

	FString FServerReceiver::GetDescription() const
	{
		if (Socket)
		{
			return Socket->GetDescription();
		}

		return InvalidReceiverName;
	}

	TSharedRef<FServerReceiver> FServerReceiver::Launch(const FString& InName, const FIPv4Endpoint& InEndpoint, FOptions InOptions)
	{
		TSharedRef<FServerReceiver> NewReceiver = MakeShared<FServerReceiver>(FPrivateToken { }, MoveTemp(InOptions));

		const bool bSocketInit = NewReceiver->InitSocket(InName, InEndpoint);
		if (bSocketInit)
		{
			if (NewReceiver->Socket)
			{
				NewReceiver->StartThread(InName, InEndpoint);
				UE_LOGF(LogOSC, Display, "OSCServer '%ls' started", *InName);
			}
		}
		else
		{
			NewReceiver->Socket.Reset();
		}

		return NewReceiver;
	}

	uint32 FServerReceiver::Run()
	{
		check(Socket);
		while (!bStopping)
		{
			if (Socket->Wait(ESocketWaitConditions::WaitForRead, Options.WaitTime))
			{
				Tick();
			}
		}

		return 0;
	}

	void FServerReceiver::StartThread(const FString& InName, const FIPv4Endpoint& InEndpoint)
	{
		check(Socket);
		check(Socket->GetSocketType() == SOCKTYPE_Datagram);

		const FString ThreadName = FString::Printf(TEXT("OSCReceiver_%s_%s"), *InName, *InEndpoint.ToString());
		Thread = TSharedPtr<FRunnableThread>(FRunnableThread::Create(
			this,
			*ThreadName,
			Options.StackSize,
			Options.Priority,
			FPlatformAffinity::GetPoolThreadMask()
		));
	}

	void FServerReceiver::Tick()
	{
		using namespace ServerReceieverPrivate;

		TRACE_CPUPROFILER_EVENT_SCOPE(FServerReceiver_Tick);

		checkf(Socket, TEXT("ServerReceiver thread should never execute when socket is invalid"));

		uint32 Size;
		while (Socket->HasPendingData(Size))
		{
			FPacketDataRef PacketData = MakeShared<TArray<uint8>>();
			PacketData->AddUninitialized(FMath::Min(Size, Options.MaxReadBufferSize));

			TSharedRef<FInternetAddr> Sender = GetSocketSubsystemChecked().CreateInternetAddr();

			int32 NumRead = 0;
			if (Socket->RecvFrom(PacketData->GetData(), PacketData->Num(), NumRead, *Sender))
			{
				ensureMsgf((uint32)NumRead <= Options.MaxReadBufferSize, TEXT("OSC Server Socket '%s' overflow"), *Socket->GetDescription());
				PacketData->RemoveAt(NumRead, PacketData->Num() - NumRead, EAllowShrinking::No);
				Options.ReceivedDataDelegate.ExecuteIfBound(StaticCastSharedRef<const TArray<uint8>>(PacketData), FIPv4Endpoint(Sender));
			}
		}
	}
} // namespace UE::OSC
