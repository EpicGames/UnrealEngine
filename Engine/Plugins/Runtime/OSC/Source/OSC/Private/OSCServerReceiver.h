// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GenericPlatform/GenericPlatformAffinity.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IPAddress.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/SingleThreadRunnable.h"
#include "OSCLog.h"
#include "OSCStream.h"
#include "OSCTypes.h"
#include "Sockets.h"
#include "UObject/NoExportTypes.h"

#include <atomic>


namespace UE::OSC
{
	using FPacketDataRef = TSharedRef<TArray<uint8>>;
	using FConstPacketDataRef = TSharedRef<const TArray<uint8>>;
	DECLARE_DELEGATE_TwoParams(FOnServerReceivedData, FConstPacketDataRef, const FIPv4Endpoint&);

	class FServerReceiver
		: public TSharedFromThis<FServerReceiver>
		, public FRunnable
		, private FSingleThreadRunnable
	{
		struct FPrivateToken { explicit FPrivateToken() = default; };

	public:
		struct FOptions
		{
			FOnServerReceivedData ReceivedDataDelegate;
			bool bMulticastLoopback = false;

			EThreadPriority Priority = EThreadPriority::TPri_AboveNormal;
			int32 StackSize = 128 * 1024;
			FTimespan WaitTime = FTimespan::FromMilliseconds(100);
			uint32 MaxReadBufferSize = 65507u;
		};

		explicit FServerReceiver(FPrivateToken, FOptions InOptions);
		virtual ~FServerReceiver();

		static TSharedRef<FServerReceiver> Launch(const FString& InName, const FIPv4Endpoint& InEndpoint, FOptions InOptions);

		FString GetDescription() const;

	protected:
		virtual uint32 Run() override;
		virtual void Tick() override;

	private:
		bool BindSocket(ISocketSubsystem& SocketSubsystem, const FIPv4Endpoint& InEndpoint, TSharedRef<FInternetAddr> RemoteAddr, const FString& InName);
		bool InitMulticast(ISocketSubsystem& SocketSubsystem, const FIPv4Endpoint& InEndpoint, TSharedRef<FInternetAddr> RemoteAddr, const FString& InName);
		bool InitSocket(const FString& InName, FIPv4Endpoint InEndpoint);
		void StartThread(const FString& InName, const FIPv4Endpoint& InEndpoint);

		std::atomic<bool> bStopping { false };

		/** The network socket. */
		FUniqueSocket Socket;

		/** Dedicated thread for server to run on. */
		TSharedPtr<FRunnableThread> Thread;

		FString InvalidReceiverName;

		const FOptions Options;
	};
} // namespace UE::OSC
