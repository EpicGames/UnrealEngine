// Copyright Epic Games, Inc. All Rights Reserved.
#include "OSCStream.h"

#include "CoreGlobals.h"
#include "OSCLog.h"
#include "Math/Color.h"


namespace UE::OSC
{
	FStream::FStream()
	{
		// Set default size to reasonable size to avoid thrashing with allocations while writing
		Data.Reserve(1024);
	}

	FStream::FStream(const uint8* InData, int32 InSize)
		: Data(InData, InSize)
		, bIsReadStream(true)
	{
	}

	const uint8* FStream::GetData() const
	{
		return Data.GetData();
	}

	int32 FStream::GetLength() const
	{
		return Data.Num();
	}

	bool FStream::HasReachedEnd() const
	{
		return Position >= Data.Num();
	}

	int32 FStream::GetPosition() const
	{
		return Position;
	}

	void FStream::SetPosition(int32 InPosition)
	{
		check(InPosition <= Data.Num());
		Position = InPosition;
	}

	TCHAR FStream::ReadChar()
	{
		uint8 Temp;
		if (Read(&Temp, 1) > 0)
		{
			UE_CLOGF(Temp > 0x7F, LogOSC, Warning, "Non-ANSI character '%u' written to OSCStream", Temp);
			return (TCHAR)Temp;
		}

		return '\0';
	}

	void FStream::WriteChar(TCHAR Char)
	{
		const uint32 Temp = FChar::ToUnsigned(Char);
		UE_CLOGF(Temp > 0x7F, LogOSC, Warning, "Non-ANSI character '%u' written to OSCStream", Temp);
		const uint8 Temp8 = (uint8)Temp;
		Write(&Temp8, 1);
	}

	FColor FStream::ReadColor()
	{
		uint32 Packed = static_cast<uint32>(ReadInt32());
		return FColor(Packed);
	}

	void FStream::WriteColor(FColor Color)
	{
	#if PLATFORM_LITTLE_ENDIAN
		uint32 Packed = Color.ToPackedABGR();
	#else // PLATFORM_LITTLE_ENDIAN
		uint32 Packed = Color.ToPackedRGBA();
	#endif // !PLATFORM_LITTLE_ENDIAN

		WriteInt32(static_cast<int32>(Packed));
	}

	int32 FStream::ReadInt32()
	{
		return ReadNumeric<int32>();
	}

	void FStream::WriteInt32(int32 Value)
	{
		WriteNumeric<int32>(Value);
	}

	double FStream::ReadDouble()
	{
		return ReadNumeric<double>();
	}

	void FStream::WriteDouble(uint64 Value)
	{
		WriteNumeric<double>(Value);

	}

	int64 FStream::ReadInt64()
	{
		return ReadNumeric<int64>();
	}

	void FStream::WriteInt64(int64 Value)
	{
		WriteNumeric<int64>(Value);
	}

	uint64 FStream::ReadUInt64()
	{
		return ReadNumeric<uint64>();
	}

	void FStream::WriteUInt64(uint64 Value)
	{
		WriteNumeric<uint64>(Value);
	}

	float FStream::ReadFloat()
	{
		return ReadNumeric<float>();
	}

	void FStream::WriteFloat(float Value)
	{
		WriteNumeric<float>(Value);
	}

	FString FStream::ReadString()
	{
		const int32 DataSize = Data.Num();
		if (HasReachedEnd() || DataSize == 0)
		{
			return FString();
		}

		// Cache init position index and push along until either at end or character read is null terminator.
		const int32 InitPosition = GetPosition();
		while (!HasReachedEnd() && ReadChar() != '\0');

		if (HasReachedEnd() && Data.Last() != '\0')
		{
			UE_LOGF(LogOSC, Error, "Invalid string when reading OSCStream: Null terminator '\0' not found");
			return FString();
		}

		// Cache end for string copy, increment to next read
		// location, and consume pad until next 4-byte boundary.
		const int32 EndPosition = Position;

		// Count includes the null terminator.
		const int32 Count = EndPosition - InitPosition;
		check(Count > 0);

		const int32 UnboundByteCount = Count % 4;
		if (UnboundByteCount != 0)
		{
			Position += 4 - UnboundByteCount;
		}

		// Exclude null terminator here; this constructor appends one.
		return FString::ConstructFromPtrSize((ANSICHAR*)(&Data[InitPosition]), Count - 1);
	}

	void FStream::WriteString(const FString& InString)
	{
		const TArray<TCHAR, FString::AllocatorType>& CharArr = InString.GetCharArray();

		int32 Count = CharArr.Num();
		if (Count == 0)
		{
			WriteChar('\0');
			Count++;
		}
		else
		{
			for (int32 i = 0; i < Count; i++)
			{
				WriteChar(CharArr[i]);
			}
		}

		// Increment & pad string with null terminator (String must
		// be packed into multiple of 4 bytes)
		check(Count > 0);
		const int32 UnboundByteCount = Count % 4;
		if (UnboundByteCount != 0)
		{
			const int32 NumPaddingZeros = 4 - UnboundByteCount;
			for (int32 i = 0; i < NumPaddingZeros; i++)
			{
				WriteChar('\0');
			}
		}
	}

	TArray<uint8> FStream::ReadBlob()
	{
		TArray<uint8> Blob;

		const int32 BlobSize = ReadInt32();
		for (int32 i = 0; i < BlobSize; i++)
		{
			Blob.Add(ReadChar());
		}

		Position = ((Position + 3) / 4) * 4; // padded

		return Blob;
	}

	void FStream::WriteBlob(TArray<uint8>& Blob)
	{
		WriteInt32(Blob.Num());
		Write(Blob.GetData(), Blob.Num());

		const int32 UnboundByteCount = Blob.Num() % 4;
		if (UnboundByteCount != 0)
		{
			const int32 NumPaddingZeros = 4 - UnboundByteCount;
			for (int32 i = 0; i < NumPaddingZeros; i++)
			{
				WriteChar('\0');
			}
		}
	}

	int32 FStream::Read(uint8* OutBuffer, int32 InSize)
	{
		check(bIsReadStream);
		const int32 DataSize = Data.Num();
		if (InSize == 0 || Position >= DataSize)
		{
			return 0;
		}

		const int32 Num = FMath::Min<int32>(InSize, DataSize - Position);
		if (Num > 0)
		{
			FMemory::Memcpy(OutBuffer, &Data[Position], Num);
			Position += Num;
		}

		return Num;
	}

	int32 FStream::Write(const uint8* InBuffer, int32 InSize)
	{
		check(!bIsReadStream);
		if (InSize <= 0)
		{
			return 0;
		}

		if (Position < Data.Num())
		{
			int32 Slack = Data.Num() - Position;
			if (InSize - Slack > 0)
			{
				Data.AddUninitialized(InSize - Slack);
			}
		}
		else
		{
			check(Position == Data.Num());
			Data.AddUninitialized(InSize);
		}

		FMemory::Memcpy(&Data[Position], InBuffer, InSize);
		Position += InSize;
		return InSize;
	}
} // namespace UE::OSC
