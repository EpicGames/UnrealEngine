// Copyright Epic Games, Inc. All Rights Reserved.

#include "OSCTypes.h"


namespace UE::OSC
{
	namespace TypesPrivate
	{
		template <typename TVariantType>
		TVariantType GetSafe(const FOSCData::FVariant& InVariant, const TVariantType& Default)
		{
			if (const TVariantType* Value = InVariant.TryGet<TVariantType>())
			{
				return *Value;
			}

			return Default;
		}
	} // namespace TypesPrivate

	const TCHAR* LexToString(EDataType DataType)
	{
		switch (DataType)
		{
			case EDataType::Blob:
				return TEXT("Blob");
			case EDataType::Char:
				return TEXT("Char");
			case EDataType::Color:
				return TEXT("Color");
			case EDataType::Double:
				return TEXT("Double");
			case EDataType::Float:
				return TEXT("Float");
			case EDataType::Infinitum:
				return TEXT("Infinitum");
			case EDataType::Int32:
				return TEXT("Int32");
			case EDataType::Int64:
				return TEXT("Int64");
			case EDataType::NilValue:
				return TEXT("Nil");
			case EDataType::String:
				return TEXT("String");
			case EDataType::Terminate:
				return TEXT("Terminate");
			case EDataType::Time:
				return TEXT("Time");

			// Both treated as bool
			case EDataType::False:
			case EDataType::True:
				return TEXT("Bool");

			default:
			{
				checkNoEntry();
				return nullptr;
			}
		}
	}

	FOSCData::FOSCData(const TArray<uint8>& Value)
		: DataType(EDataType::Blob)
	{
		Data.Set<TArray<uint8>>(Value);
	}

	FOSCData::FOSCData(TArray<uint8>&& Value)
		: DataType(EDataType::Blob)
	{
		Data.Set<TArray<uint8>>(MoveTemp(Value));
	}

	FOSCData::FOSCData(bool Value)
		: DataType(Value ? EDataType::True : EDataType::False)
	{
		Data.Set<int32>(0);
	}

	FOSCData::FOSCData(ANSICHAR Value)
		: DataType(EDataType::Char)
	{
		Data.Set<ANSICHAR>(Value);
	}

	FOSCData::FOSCData(FColor Value)
		: DataType(EDataType::Color)
	{
		Data.Set<FColor>(Value);
	}

	FOSCData::FOSCData(double Value)
		: DataType(EDataType::Double)
	{
		Data.Set<double>(Value);
	}

	FOSCData::FOSCData(float Value)
		: DataType(EDataType::Float)
	{
		Data.Set<float>(Value);
	}

	FOSCData::FOSCData(int32 Value)
		: DataType(EDataType::Int32)
	{
		Data.Set<int32>(Value);
	}

	FOSCData::FOSCData(int64 Value)
		: DataType(EDataType::Int64)
	{
		Data.Set<int64>(Value);
	}

	FOSCData::FOSCData(FString Value)
		: DataType(EDataType::String)
	{
		Data.Set<FString>(MoveTemp(Value));
	}

	FOSCData::FOSCData(uint64 Value)
		: DataType(EDataType::Time)
	{
		Data.Set<uint64>(Value);
	}

	FOSCData::FOSCData::FOSCData(EDataType DataType)
		: DataType(DataType)
	{
		Data.Set<int32>(0);
	}

	const FOSCData& FOSCData::NilData()
	{
		auto MakeInf = []()
			{
				FOSCData NilType;
				NilType.DataType = EDataType::NilValue;
				return NilType;
			};
		static const FOSCData StaticNil = MakeInf();
		return StaticNil;
	}

	const FOSCData& FOSCData::Infinitum()
	{
		auto MakeInf = []()
			{
				FOSCData Inf;
				Inf.DataType = EDataType::Infinitum;
				return Inf;
			};
		static const FOSCData InfType = MakeInf();
		return InfType;
	}

	const FOSCData& FOSCData::Terminate()
	{
		auto MakeTerm = []()
			{
				FOSCData Term;
				Term.DataType = EDataType::Terminate;
				return Term;
			};
		static const FOSCData TermType = MakeTerm();
		return TermType;
	}

	bool FOSCData::IsNil(const FOSCData& InType)
	{
		return InType.GetDataType() == EDataType::NilValue;
	}

	EDataType FOSCData::GetDataType() const
	{
		return DataType;
	}

	TArray<uint8> FOSCData::GetBlob() const
	{
		return TypesPrivate::GetSafe<TArray<uint8>>(Data, TArray<uint8>());
	}

	bool FOSCData::GetBool() const
	{
		return DataType == EDataType::True;
	}

	ANSICHAR FOSCData::GetChar() const
	{
		return TypesPrivate::GetSafe<ANSICHAR>(Data, '\0');
	}

	FColor FOSCData::GetColor() const
	{
		return TypesPrivate::GetSafe(Data, FColor::Black);
	}

	double FOSCData::GetDouble() const
	{
		return TypesPrivate::GetSafe<double>(Data, 0.0);
	}

	float FOSCData::GetFloat() const
	{
		return TypesPrivate::GetSafe<float>(Data, 0.0f);
	}

	int32 FOSCData::GetInt32() const
	{
		return TypesPrivate::GetSafe<int32>(Data, 0);
	}

	int64 FOSCData::GetInt64() const
	{
		return TypesPrivate::GetSafe<int64>(Data, 0);
	}

	FString FOSCData::GetString() const
	{
		return TypesPrivate::GetSafe(Data, FString());
	}

	uint64 FOSCData::GetTimeTag() const
	{
		return TypesPrivate::GetSafe<uint64>(Data, 0);
	}

	TArrayView<const uint8> FOSCData::GetBlobArrayView() const
	{
		if (const TArray<uint8>* BlobPtr = Data.TryGet<TArray<uint8>>())
		{
			return TArrayView<const uint8>(BlobPtr->GetData(), BlobPtr->Num());
		}

		return { };
	}

	FStringView FOSCData::GetStringView() const
	{
		if (const FString* StrPtr = Data.TryGet<FString>())
		{
			return *StrPtr;
		}

		return { };
	}
} // namespace UE::OSC
