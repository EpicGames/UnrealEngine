// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "OSCLog.h"

#include "OSCAddress.generated.h"


namespace UE::OSC
{
	extern const FString BundleTag;
	extern const FString PathSeparator;
} // namespace UE::OSC


USTRUCT(BlueprintType)
struct OSC_API FOSCAddress
{
	GENERATED_USTRUCT_BODY()

private:
	// Ordered array of container names
	TArray<FString> Containers;

	// Method name of string
	FString Method;

	/** Cached values for validity and hash */
	bool bIsValidPattern;
	bool bIsValidPath;
	uint32 Hash;

	void CacheAggregates();

public:
	FOSCAddress();
	FOSCAddress(const FString& InValue);

	/** Returns whether address is valid pattern */
	bool IsValidPattern() const;

	/** Returns whether address is valid path */
	bool IsValidPath() const;

	/** Returns if this address is a pattern that matches InAddress' path.
	  * If passed address is not a valid path, returns false.
	  */
	bool Matches(const FOSCAddress& InAddress) const;

	/** Pushes container onto end of address' ordered array of containers */
	bool PushContainer(FString Container);

	/** Pushes containers onto end of address' ordered array of containers */
	bool PushContainers(TArray<FString> NewContainers);

	/** Pops container from ordered array of containers */
	FString PopContainer();

	/** Pops container from ordered array of containers */
	bool PopContainer(FString& OutContainer);

	/** Pops containers off end of address' ordered array of containers */
	TArray<FString> PopContainers(int32 NumToPop);

	/** Pops containers off end of address' ordered array of containers */
	bool PopContainers(int32 NumToPop, TArray<FString>& OutContainers);

	/** Removes containers from container array at index until count */
	bool RemoveContainers(int32 InIndex, int32 InCount);

	/** Clears ordered array of containers */
	void ClearContainers();

	/** Get method name of address */
	const FString& GetMethod() const;

	/** Returns numeric prefix of OSC method if is numeric */
	bool GetNumericPrefix(int32& OutPrefix) const;

	/** Returns numeric suffix of OSC method if is numeric */
	bool GetNumericSuffix(int32& OutSuffix) const;

	/** Sets the address to the given array of containers and provided method */
	bool Set(TArray<FString> NewContainers, FString NewMethod);

	/** Sets the method name of address */
	bool SetMethod(FString NewMethod);

	/** Returns container path of OSC address in the form '/Container1/Container2' */
	FString GetContainerPath() const;

	/** Returns container at provided Index.  If Index is out-of-bounds, returns empty string. */
	FString GetContainer(int32 Index) const;

	/** Builds referenced array of address of containers in order */
	void GetContainers(TArray<FString>& OutContainers) const;

	/** Returns full path of OSC address in the form '/Container1/Container2/Method' */
	FString GetFullPath() const;

	bool operator== (const FOSCAddress& InAddress) const
	{
		if (Hash != InAddress.Hash)
		{
			return false;
		}

		if (InAddress.Containers.Num() != Containers.Num())
		{
			return false;
		}

		if (InAddress.Method != Method)
		{
			return false;
		}

		for (int32 i = 0; i < Containers.Num(); ++i)
		{
			if (InAddress.Containers[i] != Containers[i])
			{
				return false;
			}
		}

		return true;
	}

	FOSCAddress& operator/= (const FOSCAddress& InAddress)
	{
		Containers.Add(Method);
		Containers.Append(InAddress.Containers);

		Method = InAddress.Method;

		CacheAggregates();
		return *this;
	}

	FOSCAddress operator/ (const FOSCAddress& InAddress) const
	{
		FOSCAddress ToReturn = *this;
		ToReturn /= InAddress;
		return ToReturn;
	}

	friend uint32 GetTypeHash(const FOSCAddress& InAddress)
	{
		return InAddress.Hash;
	}

private:
	bool PushContainersInternal(TArray<FString>&& InContainers);
};
