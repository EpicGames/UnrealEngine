// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "OSCPacket.h"
#include "OSCBundle.generated.h"


USTRUCT(BlueprintType)
struct OSC_API FOSCBundle
{
	GENERATED_USTRUCT_BODY()

	FOSCBundle();
	explicit FOSCBundle(const TSharedRef<UE::OSC::IPacket>& InPacket);

	UE_DEPRECATED(5.5, "Use explicit shared ref constructor instead")
	FOSCBundle(const TSharedPtr<UE::OSC::IPacket>& InPacket);

	~FOSCBundle() = default;

	UE_DEPRECATED(5.5, "Use shared ref setter instead")
	void SetPacket(TSharedPtr<UE::OSC::IPacket>& InPacket);

	void SetPacket(const TSharedRef<UE::OSC::IPacket>& InPacket);

	UE_DEPRECATED(5.5, "Use shared ref getter instead")
	const TSharedPtr<UE::OSC::IPacket>& GetPacket() const;

	const TSharedRef<UE::OSC::IPacket>& GetPacketRef() const;

private:
	TSharedRef<UE::OSC::IPacket> Packet;
};