// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/UniquePtr.h"
#include "UObject/Object.h"

#include "OSCMessage.h"
#include "OSCBundle.h"

#include "OSCClient.generated.h"


namespace UE::OSC
{
	/** Interface for internal network implementation of sending OSC messages & bundles as a client. */
	class OSC_API IClientProxy
	{
	public:
		// Creates a new client proxy that can be used by any system where the provided dispatch callback is called on a worker thread.
		static TUniquePtr<IClientProxy> Create(const FString& ClientName);

		virtual ~IClientProxy() = default;

		UE_DEPRECATED(5.5, "Use GetSendIPEndpoint instead")
		virtual void GetSendIPAddress(FString& InIPAddress, int32& Port) const = 0;

		UE_DEPRECATED(5.5, "Use SetSendIPEndpoint instead")
		virtual bool SetSendIPAddress(const FString& InIPAddress, const int32 Port) = 0;

		virtual const FIPv4Endpoint& GetSendIPEndpoint() const = 0;
		virtual void SetSendIPEndpoint(const FIPv4Endpoint& InEndpoint) = 0;

		virtual bool IsActive() const = 0;

		virtual void SendMessage(const FOSCMessage& Message) = 0;
		virtual void SendBundle(const FOSCBundle& Bundle) = 0;

		virtual void Stop() = 0;
	};
} // namespace UE::OSC

// For backward compat.  To be deprecated
class OSC_API IOSCClientProxy : public UE::OSC::IClientProxy
{
public:
	UE_DEPRECATED(5.5, "Use UE::OSC::IClientProxy instead")
	IOSCClientProxy() = default;

	virtual ~IOSCClientProxy() = default;
};

UCLASS(BlueprintType)
class OSC_API UOSCClient : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	void Connect();

	bool IsActive() const;

	/** Gets the OSC Client IP address and port. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void GetSendIPAddress(UPARAM(ref) FString& IPAddress, UPARAM(ref) int32& Port);

	/** Sets the OSC Client IP address and port. Returns whether
	  * address and port was successfully set. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	bool SetSendIPAddress(const FString& IPAddress, const int32 Port);

	/** Send OSC message to  a specific address. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void SendOSCMessage(UPARAM(ref) FOSCMessage& Message);

	/** Send OSC Bundle over the network. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void SendOSCBundle(UPARAM(ref) FOSCBundle& Bundle);

protected:
	void BeginDestroy() override;

	/** Stop and tidy up network socket. */
	void Stop();
	
	/** Pointer to internal implementation of client proxy */
	TUniquePtr<UE::OSC::IClientProxy> ClientProxy;
};
