// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "OSCAddress.h"
#include "OSCBundle.h"
#include "OSCMessage.h"
#include "UObject/ObjectMacros.h"


#include "OSCManager.generated.h"


// Forward Declarations
class UOSCServer;
class UOSCClient;

namespace UE::OSC
{
	OSC_API int32 GetDefaultClientPort();
	OSC_API int32 GetDefaultServerPort();
}

UCLASS()
class OSC_API UOSCManager : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Creates an OSC Server.  If ReceiveIPAddress left empty (or '0'),
	// attempts to use LocalHost IP address. If StartListening set,
	// immediately begins listening on creation.
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	static UOSCServer* CreateOSCServer(FString ReceiveIPAddress, int32 Port, bool bMulticastLoopback, bool bStartListening, FString ServerName, UObject* Outer = nullptr);

	// Creates an OSC Client.  If SendIPAddress left empty (or '0')
	// attempts to use LocalHost IP address.
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	static UOSCClient* CreateOSCClient(FString SendIPAddress, int32 Port, FString ClientName, UObject* Outer = nullptr);

	/** Adds provided message packet to bundle. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add OSC Message to Bundle", Keywords = "osc message bundle"))
	static UPARAM(DisplayName = "Bundle") FOSCBundle& AddMessageToBundle(const FOSCMessage& Message, UPARAM(ref) FOSCBundle& Bundle);

	/** Adds bundle packet to bundle. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add OSC Bundle to Bundle", Keywords = "osc message"))
	static UPARAM(DisplayName = "OutBundle") FOSCBundle& AddBundleToBundle(const FOSCBundle& InBundle, UPARAM(ref) FOSCBundle& OutBundle);

	/** Fills array with child bundles found in bundle. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Bundles From Bundle", Keywords = "osc bundle"))
	static UPARAM(DisplayName = "Bundles") TArray<FOSCBundle> GetBundlesFromBundle(const FOSCBundle& Bundle);

	/** Returns message found in bundle at ordered index. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message From Bundle At Index", Keywords = "osc bundle message"))
	static UPARAM(DisplayName = "Message") FOSCMessage GetMessageFromBundle(const FOSCBundle& Bundle, int32 Index, bool& bSucceeded);

	/** Fills array with messages found in bundle. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Messages From Bundle", Keywords = "osc bundle message"))
	static UPARAM(DisplayName = "Messages") TArray<FOSCMessage> GetMessagesFromBundle(const FOSCBundle& Bundle);

	/** Clears provided message of all arguments. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Clear OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& ClearMessage(UPARAM(ref) FOSCMessage& Message);

	/** Clears provided bundle of all internal messages/bundle packets. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Clear OSC Bundle", Keywords = "osc message"))
	static UPARAM(DisplayName = "Bundle") FOSCBundle& ClearBundle(UPARAM(ref) FOSCBundle& Bundle);

	/** Adds float value to end of OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add Float to OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& AddFloat(UPARAM(ref) FOSCMessage& Message, float Value);

	/** Adds Int32 value to end of OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add Integer to OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& AddInt32(UPARAM(ref) FOSCMessage& Message, int32 Value);

	/** Adds Int64 value to end of OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add Integer (64-bit) to OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& AddInt64(UPARAM(ref) FOSCMessage& Message, int64 Value);

	/** Adds address (packed as string) value to end of OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add OSC Address (As String) to OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& AddAddress(UPARAM(ref) FOSCMessage& Message, const FOSCAddress& Value);

	/** Adds string value to end of OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add String to OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& AddString(UPARAM(ref) FOSCMessage& Message, UPARAM(ref) FString Value);

	/** Adds blob value to end of OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add Blob to OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& AddBlob(UPARAM(ref) FOSCMessage& Message, const TArray<uint8>& Value);

	/** Adds boolean value to end of OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Add Bool to OSC Message", Keywords = "osc message"))
	static UPARAM(DisplayName = "Message") FOSCMessage& AddBool(UPARAM(ref) FOSCMessage& Message, bool Value);

	/** Sets Value to address at provided Index in OSCMessage if in bounds and OSC type matches 'String' (Does NOT return address of message, rather
	 * string packed in message and casts to OSC address). Returns if string found at index and is valid OSC address path.
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Address At Index", Keywords = "osc message"))
	static UPARAM(DisplayName = "Succeeded") bool GetAddress(const FOSCMessage& Message, const int32 Index, FOSCAddress& Value);

	/** Returns all strings that are valid address paths in order received from OSCMessage (Does NOT include address of message, just
	 * strings packed in message that are valid paths).
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Addresses", Keywords = "osc message"))
	static void GetAllAddresses(const FOSCMessage& Message, TArray<FOSCAddress>& Values);

	/**
	 * Set Value to float at provided Index in OSCMessage if in bounds and type matches
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Float At Index", Keywords = "osc message"))
	static UPARAM(DisplayName = "Succeeded") bool GetFloat(const FOSCMessage& Message, const int32 Index, float& Value);

	/** Returns all float values in order of received from OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Floats", Keywords = "osc message"))
	static void GetAllFloats(const FOSCMessage& Message, TArray<float>& Values);

	/** Set Value to integer at provided Index in OSCMessage if in bounds and type matches */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Integer at Index", Keywords = "osc message"))
	static UPARAM(DisplayName = "Succeeded") bool GetInt32(const FOSCMessage& Message, const int32 Index, int32& Value);

	/** Returns all integer values in order of received from OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Integers", Keywords = "osc message"))
	static void GetAllInt32s(const FOSCMessage& Message, TArray<int32>& Values);

	/** Set Value to Int64 at provided Index in OSCMessage if in bounds and type matches */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Integer (64-bit) at Index", Keywords = "osc message"))
	static UPARAM(DisplayName = "Succeeded") bool GetInt64(const FOSCMessage& Message, const int32 Index, int64& Value);

	/** Returns all Int64 values in order of received from OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Integers (64-bit)", Keywords = "osc message"))
	static void GetAllInt64s(const FOSCMessage& Message, TArray<int64>& Values);

	/** Set Value to string at provided Index in OSCMessage if in bounds and type matches */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message String at Index", Keywords = "osc message"))
	static UPARAM(DisplayName = "Succeeded") bool GetString(const FOSCMessage& Message, const int32 Index, FString& Value);

	/** Returns all string values in order of received from OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Strings", Keywords = "osc message"))
	static void GetAllStrings(const FOSCMessage& Message, TArray<FString>& Values);

	/** Sets Value to boolean at provided Index from OSCMessage if in bounds and type matches */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Bool At Index", Keywords = "osc message"))
	static UPARAM(DisplayName = "Succeeded") bool GetBool(const FOSCMessage& Message, const int32 Index, bool& Value);

	/** Returns all boolean values in order of received from OSCMessage */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Bools", Keywords = "osc message"))
	static void GetAllBools(const FOSCMessage& Message, TArray<bool>& Values);

	/** Sets Value to blob at provided Index from OSCMessage if in bounds and type matches */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	static UPARAM(DisplayName = "Succeeded") bool GetBlob(const FOSCMessage& Message, const int32 Index, TArray<uint8>& Value);

	/** Returns whether OSC Address is valid path */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Is OSC Address Valid Path", Keywords = "valid osc address path address"))
	static bool OSCAddressIsValidPath(const FOSCAddress& Address);

	/** Returns whether OSC Address is valid pattern to match against */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Is OSC Address Valid Pattern", Keywords = "valid osc address pattern address"))
	static bool OSCAddressIsValidPattern(const FOSCAddress& Address);

	/* Converts string to OSC Address */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	static UPARAM(DisplayName = "Address") FOSCAddress ConvertStringToOSCAddress(const FString& String);

	/** Returns if address pattern matches the provided address path.
	  * If passed address is not a valid path, returns false.
	  */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "OSC Address Path Matches Pattern", Keywords = "matches osc address path address"))
	static UPARAM(DisplayName = "Is Match") bool OSCAddressPathMatchesPattern(const FOSCAddress& Pattern, const FOSCAddress& Path);

	/** Finds an object with the given OSC Address in path form, where containers correspond to path folders and the the address method to the object's name.
	  * Only supports parent objects.
	  */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Find Object at OSC Address", Keywords = "osc address path uobject"))
	static UPARAM(DisplayName = "Object") UObject* FindObjectAtOSCAddress(const FOSCAddress& Address);

	/** Converts object path to OSC Address, converting folders to address containers and the object's name to the address method.
	 * Only supports parent objects (See UObjectBaseUtility::GetPathName and UObjectBaseUtility::GetFullName).
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Convert Object Path to OSC Address", Keywords = "osc address path uobject"))
	static UPARAM(DisplayName = "Address") FOSCAddress OSCAddressFromObjectPath(UObject* Object);

	/** Converts object path string to OSC Address, converting folders to address containers and the object's name to the address method.
	 * Only supports parent objects (See UObjectBaseUtility::GetPathName and UObjectBaseUtility::GetFullName).
	 */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Convert Object Path (String) to OSC Address", Keywords = "osc address path uobject"))
	static UPARAM(DisplayName = "Address") FOSCAddress OSCAddressFromObjectPathString(const FString& PathName);

	/** Converts OSC Address to an object path. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Convert OSC Address to Object Path", Keywords = "osc address path uobject"))
	static UPARAM(DisplayName = "Path") FString ObjectPathFromOSCAddress(const FOSCAddress& Address);

	/** Pushes container onto address' ordered array of containers */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Push Container to OSC Address", Keywords = "push osc address container"))
	static UPARAM(DisplayName = "Address") FOSCAddress& OSCAddressPushContainer(UPARAM(ref) FOSCAddress& Address, const FString& Container);

	/** Pushes container onto address' ordered array of containers */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Push Container Array to OSC Address", Keywords = "push osc address container"))
	static UPARAM(DisplayName = "Address") FOSCAddress& OSCAddressPushContainers(UPARAM(ref) FOSCAddress& Address, const TArray<FString>& Containers);

	/** Pops container from ordered array of containers. If no containers, returns empty string */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Pop Container from OSC Address", Keywords = "pop osc address container"))
	static UPARAM(DisplayName = "Container") FString OSCAddressPopContainer(UPARAM(ref) FOSCAddress& Address);

	/** Pops container from ordered array of containers. If NumContainers is greater than or equal to the number of containers in address, returns all containers. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Pop Containers from OSC Address", Keywords = "pop osc address container"))
	static UPARAM(DisplayName = "Containers") TArray<FString> OSCAddressPopContainers(UPARAM(ref) FOSCAddress& Address, int32 NumContainers);

	/** Remove containers from ordered array of containers at index up to count of containers. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Remove Containers from OSC Address", Keywords = "remove osc address container"))
	static UPARAM(DisplayName = "Address") FOSCAddress& OSCAddressRemoveContainers(UPARAM(ref) FOSCAddress& Address, int32 Index, int32 Count);

	/* Returns copy of message's OSC Address */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Message Address", Keywords = "osc address"))
	static UPARAM(DisplayName = "Address") FOSCAddress GetOSCMessageAddress(const FOSCMessage& Message);

	/** Sets the OSC Address of the provided message */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	static UPARAM(DisplayName = "Message") FOSCMessage& SetOSCMessageAddress(UPARAM(ref) FOSCMessage& Message, const FOSCAddress& Address);

	/** Returns the OSC Address container at the provided 'Index.' Returns empty string if index is out-of-bounds. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Address Container At Index", Keywords = "osc address container path"))
	static UPARAM(DisplayName = "Container") FString GetOSCAddressContainer(const FOSCAddress& Address, const int32 Index);

	/** Builds referenced array of address of containers in order */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Address Containers", Keywords = "osc address container path"))
	static UPARAM(DisplayName = "Containers") TArray<FString> GetOSCAddressContainers(const FOSCAddress& Address);

	/** Returns full path of OSC address in the form '/Container1/Container2/Method' */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	static UPARAM(DisplayName = "Path") FString GetOSCAddressContainerPath(const FOSCAddress& Address);

	/** Returns full path of OSC address in the form '/Container1/Container2' */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Convert OSC Address To String"))
	static UPARAM(DisplayName = "Full Path") FString GetOSCAddressFullPath(const FOSCAddress& Address);

	/** Returns method name of OSC Address provided */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Get OSC Address Method"))
	static UPARAM(DisplayName = "Method") FString GetOSCAddressMethod(const FOSCAddress& Address);

	/** Clears containers of OSC Address provided */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Clear OSC Address Containers", Keywords = "osc address clear"))
	static UPARAM(DisplayName = "Address") FOSCAddress& ClearOSCAddressContainers(UPARAM(ref) FOSCAddress& Address);

	/** Sets the method name of the OSC Address provided */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (DisplayName = "Set OSC Address Method", Keywords = "osc method"))
	static UPARAM(DisplayName = "Address") FOSCAddress& SetOSCAddressMethod(UPARAM(ref) FOSCAddress& Address, const FString& Method);
};

