// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OSCLog.h"
#include "OSCPacket.h"
#include "OSCStream.h"
#include "OSCTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "OSCMessage.generated.h"


USTRUCT(BlueprintType)
struct OSC_API FOSCMessage
{
	GENERATED_USTRUCT_BODY()

	FOSCMessage();
	FOSCMessage(FOSCAddress Address, TArray<UE::OSC::FOSCData> Args);
	FOSCMessage(const TSharedRef<UE::OSC::IPacket>& InPacket);

	UE_DEPRECATED(5.5, "Use shared ref ctor instead")
	FOSCMessage(const TSharedPtr<UE::OSC::IPacket>& InPacket);

	~FOSCMessage() = default;

	// Returns arguments, asserting if the message's packet is unset.
	const TArray<UE::OSC::FOSCData>& GetArgumentsChecked() const;

	UE_DEPRECATED(5.5, "Use shared ref setter instead")
	void SetPacket(TSharedPtr<UE::OSC::IPacket>& InPacket);

	void SetPacket(TSharedRef<UE::OSC::IPacket>& InPacket);

	UE_DEPRECATED(5.5, "Use shared ref getter instead")
	const TSharedPtr<UE::OSC::IPacket>& GetPacket() const;

	const TSharedRef<UE::OSC::IPacket>& GetPacketRef() const;

	bool SetAddress(const FOSCAddress& InAddress);
	const FOSCAddress& GetAddress() const;

private:
	TSharedRef<UE::OSC::IPacket> Packet;
};