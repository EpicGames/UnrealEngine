// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "OSCTypes.h"
#include "OSCStream.h"
#include "OSCAddress.h"


namespace UE::OSC
{
	// Forward Declarations
	class FStream;

	class OSC_API IPacket : public TSharedFromThis<IPacket>
	{
	public:
		IPacket() = default;
		virtual ~IPacket() = default;

		/** Write packet data into stream */
		virtual void WriteData(FStream& OutStream) = 0;
	
		/** Read packet data from stream */
		virtual void ReadData(FStream& OutStream) = 0;
	
		/** Returns true if packet is message */
		virtual bool IsMessage() = 0;
	
		/** Returns true if packet is bundle */
		virtual bool IsBundle() = 0;

		UE_DEPRECATED(5.5, "Packet port can now be accessed via GetIPEndpoint() call.")
		virtual const FString& GetIPAddress() const
		{
			static FString Addr;
			Addr = GetIPEndpoint().ToString();
			return Addr;
		}

		UE_DEPRECATED(5.5, "Packet address can now be accessed via GetIPEndpoint() call.")
		virtual uint16 GetPort() const
		{
			return GetIPEndpoint().Port;
		};

		virtual const FIPv4Endpoint& GetIPEndpoint() const = 0;

		/** Create an OSC packet according to the input data. */
		static TSharedPtr<IPacket> CreatePacket(const uint8* InPacketType, const FIPv4Endpoint& InEndpoint);
	};
} // namespace UE::OSC

// Exists for back compat.  To be deprecated
class OSC_API IOSCPacket : public UE::OSC::IPacket
{
public:
	UE_DEPRECATED(5.5, "Use UE::OSC::IPacket instead")
	IOSCPacket() = default;

	virtual ~IOSCPacket() = default;
};
