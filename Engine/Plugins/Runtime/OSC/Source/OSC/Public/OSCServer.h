// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Async/TaskGraphInterfaces.h"
#include "Containers/Queue.h"
#include "Containers/SpscQueue.h"
#include "Containers/Set.h"
#include "Containers/Ticker.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "OSCBundle.h"
#include "OSCMessage.h"
#include "OSCPacket.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"

#include "OSCServer.generated.h"

// Forward Declarations
class FSocket;
class UOSCServer;


// Delegates
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOSCReceivedMessageEvent, const FOSCMessage&, Message, const FString&, IPAddress, int32, Port);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOSCReceivedMessageNativeEvent, const FOSCMessage&, const FString& /*IPAddress*/, uint16 /*Port*/);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FOSCDispatchMessageEvent, const FOSCAddress&, AddressPattern, const FOSCMessage&, Message, const FString&, IPAddress, int32, Port);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOSCReceivedBundleEvent, const FOSCBundle&, Bundle, const FString&, IPAddress, int32, Port);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOSCReceivedBundleNativeEvent, const FOSCBundle&, const FString& /*IPAddress*/, uint16 /*Port*/);
DECLARE_DYNAMIC_DELEGATE_FourParams(FOSCDispatchMessageEventBP, const FOSCAddress&, AddressPattern, const FOSCMessage&, Message, const FString&, IPAddress, int32, Port);

DECLARE_STATS_GROUP(TEXT("OSC Commands"), STATGROUP_OSCNetworkCommands, STATCAT_Advanced);


namespace UE::OSC
{
	// Forward Declarations
	class IServerProxy;

	/** Interface for internal networking implementation.  See UOSCServer for details */
	class OSC_API IServerProxy
	{
	public:
		using FOnDispatchPacket = TUniqueFunction<void(TSharedRef<UE::OSC::IPacket>)>;

		// Creates a new server proxy that can be used by any system where the provided dispatch callback is called on a worker thread.
		static TSharedPtr<IServerProxy> Create();

		virtual ~IServerProxy() { }

		// Returns whether or not packet can be processed, i.e. is valid and allowlisted.
		virtual bool CanProcessPacket(TSharedRef<UE::OSC::IPacket> Packet) const = 0;

		// Returns debug description of server proxy
		virtual FString GetDescription() const = 0;

		UE_DEPRECATED(5.5, "Use GetIPEndpoint instead")
		virtual FString GetIpAddress() const { return { }; }

		virtual const FIPv4Endpoint& GetIPEndpoint() const = 0;

		UE_DEPRECATED(5.5, "Use GetIPEndpoint instead")
		virtual int32 GetPort() const { return GetIPEndpoint().Port; }

		// Returns whether or not loopback is enabled
		virtual bool GetMulticastLoopback() const = 0;

		// Returns whether or not the server is currently active (listening)
		virtual bool IsActive() const = 0;

		// Starts the server, causing it to actively listen and dispatch OSC messages
		virtual void Listen(const FString& ServerName) = 0;

		UE_DEPRECATED(5.5, "Use SetIPEndpoint instead")
		virtual bool SetAddress(const FString& InReceiveIPAddress, int32 InPort);

		// Sets the current server's endpoint.  Ignores request and returns false if server
		// is currently active.
		virtual bool SetIPEndpoint(const FIPv4Endpoint& InEndpoint) = 0;

		// Sets whether or not loopback is enabled.  Returns false and request is ignored
		// if server is currently active.
		virtual bool SetMulticastLoopback(bool bInMulticastLoopback) = 0;

		// Sets dispatch function to be called when OSC packet is received (thread safe
		// and can be mutated while server is running)
		virtual void SetOnDispatchPacket(TSharedPtr<FOnDispatchPacket> OnDispatch) { };

#if WITH_EDITOR
		UE_DEPRECATED(5.5, "ServerProxies are no longer independently ticked objects only and can now be accessed by threads other than the GameThread via the 'SetOnDispatchPacket' setter.")
		virtual void SetTickableInEditor(bool bInTickInEditor) { };
#endif // WITH_EDITOR

		// Stops the server
		virtual void Stop() = 0;

		UE_DEPRECATED(5.5, "AllowList is now managed as IPv4Endpoints. Use API that works with endpoint struct directly")
		virtual void AddClientToAllowList(const FString& InIPAddress) { }

		UE_DEPRECATED(5.5, "AllowList is now managed as IPv4Endpoints. Use API that works with endpoint struct directly")
		virtual void RemoveClientFromAllowList(const FString& IPAddress) { }

		UE_DEPRECATED(5.5, "AllowList is now managed as IPv4Endpoints. Use API that works with endpoint struct directly")
		virtual void ClearClientAllowList() { }

		UE_DEPRECATED(5.5, "AllowList is now managed as IPv4Endpoints. Use API that works with endpoint struct directly")
		virtual TSet<FString> GetClientAllowList() const { return { }; }

		// Adds the given endpoint to the client allow list
		virtual void AddClientEndpointToAllowList(const FIPv4Endpoint& InIPv4Endpoint) = 0;

		// Removes the given endpoint from the client allow list
		virtual void RemoveClientEndpointFromAllowList(const FIPv4Endpoint& InIPv4Endpoint) = 0;

		// Empties the given address to the client allow list
		virtual void ClearClientEndpointAllowList() = 0;

		// Returns Client Address allow list if it is enabled, else returns nullptr
		virtual const TSet<FIPv4Endpoint>& GetClientEndpointAllowList() const = 0;

		// Sets whether or not allow list is active.  If disabled, entries are not cleared
		// however GetClientAddressAllowList will no longer return a valid reference to the
		// given set of allow list members.
		virtual void SetFilterClientsByAllowList(bool bEnabled) = 0;
	};
} // namespace UE::OSC


// For backward compat.  To be deprecated
class OSC_API IOSCServerProxy : public UE::OSC::IServerProxy
{
public:
	UE_DEPRECATED(5.5, "Use UE::OSC::IServerProxy instead")
	IOSCServerProxy() = default;

	virtual ~IOSCServerProxy() = default;
};


UCLASS(BlueprintType)
class OSC_API UOSCServer : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Gets whether or not to loopback if ReceiveIPAddress provided is multicast. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	bool GetMulticastLoopback() const;

	/** Returns whether server is actively listening to incoming messages. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	bool IsActive() const;

	/** Sets the IP address and port to listen for OSC data. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void Listen();

	/** Set the address and port of server. Fails if server is currently active. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	bool SetAddress(const FString& ReceiveIPAddress, int32 Port);

	/** Set whether or not to loopback if ReceiveIPAddress provided is multicast. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void SetMulticastLoopback(bool bMulticastLoopback);

	/** Stop and tidy up network socket. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void Stop();

	/** Event that gets called when an OSC message is received. */
	UPROPERTY(BlueprintAssignable, Category = "Audio|OSC")
	FOSCReceivedMessageEvent OnOscMessageReceived;

	/** Native event that gets called when an OSC message is received. */
	FOSCReceivedMessageNativeEvent OnOscMessageReceivedNative;

	/** Event that gets called when an OSC bundle is received. */
	UPROPERTY(BlueprintAssignable, Category = "Audio|OSC")
	FOSCReceivedBundleEvent OnOscBundleReceived;

	/** Native event that gets called when an OSC bundle is received. */
	FOSCReceivedBundleNativeEvent OnOscBundleReceivedNative;

	/** When set to true, server will only process received 
	  * messages from allowlisted clients.
	  */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void SetAllowlistClientsEnabled(bool bEnabled);

	/** Adds client to allowlist of clients to listen for. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void AddAllowlistedClient(const FString& IPAddress, int32 IPPort = 0);

	/** Removes allowlisted client to listen for. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void RemoveAllowlistedClient(const FString& IPAddress, int32 IPPort = 0);

	/** Clears client allowlist to listen for. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void ClearAllowlistedClients();

	/** Returns the IP for the server if connected as a string. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	FString GetIpAddress(bool bIncludePort) const;

	/** Returns the port for the server if connected. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	int32 GetPort() const;

	/** Returns set of allowlisted endpoint clients as strings with (optional) port included. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	TSet<FString> GetAllowlistedClients(bool bIncludePort = false) const;

	/** Adds event to dispatch when OSCAddressPattern is matched. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void BindEventToOnOSCAddressPatternMatchesPath(const FOSCAddress& OSCAddressPattern, const FOSCDispatchMessageEventBP& Event);

	/** Unbinds specific event from OSCAddress pattern. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void UnbindEventFromOnOSCAddressPatternMatchesPath(const FOSCAddress& OSCAddressPattern, const FOSCDispatchMessageEventBP& Event);

	/** Removes OSCAddressPattern from sending dispatch events. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void UnbindAllEventsFromOnOSCAddressPatternMatchesPath(const FOSCAddress& OSCAddressPattern);

	/** Removes all events from OSCAddressPatterns to dispatch. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	void UnbindAllEventsFromOnOSCAddressPatternMatching();

	/** Returns set of OSCAddressPatterns currently listening for matches to dispatch. */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC")
	TArray<FOSCAddress> GetBoundOSCAddressPatterns() const;

#if WITH_EDITOR
	/** Set whether server instance can be ticked in-editor (editor only and available to blueprint
	  * for use in editor utility scripts/script actions).
	  */
	UFUNCTION(BlueprintCallable, Category = "Audio|OSC", meta = (Deprecated = "5.5", DeprecationMessage = "Servers are now implemented as dispatchers, which are pumped on an async task upon calling 'Listen' at a provided rate (i.e. no longer 'ticked' on the game thread). See 'SetUpdateRate'"))
	void SetTickInEditor(bool bInTickInEditor);
#endif // WITH_EDITOR

	UE_DEPRECATED(5.5, "Clearing packets directly is not thread-safe and no longer supported.")
	void ClearPackets();

	UE_DEPRECATED(5.5, "Enqueuing packets is now handled privately")
	void EnqueuePacket(TSharedPtr<UE::OSC::IPacket> InPacket) { }

	UE_DEPRECATED(5.5, "Pumping packets is now handled privately")
	void PumpPacketQueue(const TSet<uint32>* InAllowlistedClients) { }

protected:
	virtual void BeginDestroy() override;
	virtual void PostInitProperties() override;

private:
	void ClearPacketsInternal();

	using FPacketQueue = TSpscQueue<TSharedPtr<UE::OSC::IPacket>>;
	void PumpPacketQueue();

	/** Broadcasts provided bundle received to be dispatched on the GameThread */
	void BroadcastBundle(const FOSCBundle& InBundle);

	/** Broadcasts provided message received to be dispatched on the GameThread */
	void BroadcastMessage(const FOSCMessage& InMessage);

	/** Pointer to internal implementation of server proxy */
	TSharedPtr<UE::OSC::IServerProxy> ServerProxy;

	/** Queue stores incoming OSC packet requests to process on the game thread. */
	TSharedPtr<FPacketQueue> OSCPackets;

	/** Address pattern hash to check against when dispatching incoming messages */
	TMap<FOSCAddress, FOSCDispatchMessageEvent> AddressPatterns;

	FTSTicker::FDelegateHandle TickHandle;
};
