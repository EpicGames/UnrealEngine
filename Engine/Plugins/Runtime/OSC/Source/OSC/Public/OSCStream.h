// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "UObject/NoExportTypes.h"


namespace UE::OSC
{
	class OSC_API FStream
	{
	public:
		FStream();
		FStream(const uint8* InData, int32 InSize);

		/** Returns the stream buffer data */
		const uint8* GetData() const;
	
		/** Returns the stream's length */
		int32 GetLength() const;
	
		/** Returns true if stream has reached the end, false if not */
		bool HasReachedEnd() const;
	
		/** Get current stream position. */
		int32 GetPosition() const;

		/** Set stream position. */
		void SetPosition(int32 InPosition);

		/** Read Char from the stream */
		TCHAR ReadChar();

		/** Write an ansi char into the stream */
		void WriteChar(TCHAR Char);

		/** Write Color into the stream */
		void WriteColor(FColor Color);

		/** Read Color from the stream */
		FColor ReadColor();

		/** Read Int32 from the stream */
		int32 ReadInt32();

		/** Write Int32 into the stream */
		void WriteInt32(int32 Value);

		/** Read Double from the stream */
		double ReadDouble();

		/** Write Double into the stream */
		void WriteDouble(uint64 Value);
	
		/** Read Int64 from the stream */
		int64 ReadInt64();

		/** Write Int64 into the stream */
		void WriteInt64(int64 Value);

		/** Read UInt64 from the stream */
		uint64 ReadUInt64();

		/** Write UInt64 into the stream */
		void WriteUInt64(uint64 Value);

		/** Read Float from the stream */
		float ReadFloat();
		
		/** Write Int64 into the stream */
		void WriteFloat(float Value);
	
		/** Read String from the stream */
		FString ReadString();

		/** Write String into the stream */
		void WriteString(const FString& String);
	
		/** Read blob from the stream */
		TArray<uint8> ReadBlob();

		/** Write blob into the stream */
		void WriteBlob(TArray<uint8>& Blob);

	private:
		template <typename TNumericType>
		TNumericType ReadNumeric()
		{
			constexpr int32 NumBytes = sizeof(TNumericType);
			uint8 Temp[NumBytes];
			if (Read(Temp, NumBytes) == NumBytes)
			{
#if PLATFORM_LITTLE_ENDIAN
				union
				{
					TNumericType Number;
					uint8 Data[sizeof(TNumericType)];
				} Value;

				for (int32 Index = 0; Index < NumBytes; ++Index)
				{
					Value.Data[Index] = Temp[NumBytes - Index - 1];
				}

				return Value.Number;
#else // PLATFORM_LITTLE_ENDIAN
				return *(TNumericType*)Temp;
#endif // !PLATFORM_LITTLE_ENDIAN
			}

			return 0;
		}

		template <typename TNumericType>
		void WriteNumeric(TNumericType InValue)
		{
			constexpr int32 NumBytes = sizeof(TNumericType);
			uint8 Temp[NumBytes];

#if PLATFORM_LITTLE_ENDIAN
			union
			{
				TNumericType Number;
				uint8 Data[sizeof(TNumericType)];
			} Value;

			Value.Number = InValue;

			for (int32 Index = 0; Index < NumBytes; ++Index)
			{
				Temp[NumBytes - Index - 1] = Value.Data[Index];
			}
#else // PLATFORM_LITTLE_ENDIAN
			return *(TNumericType*)(Temp) = InValue;
#endif // !PLATFORM_LITTLE_ENDIAN

			Write(Temp, NumBytes);
		}

		/** Read data from stream into a provided buffer. */
		int32 Read(uint8* OutBuffer, int32 InSize);
	
		/** Write data into stream from provided buffer. */
		int32 Write(const uint8* InBuffer, int32 InSize);

		/** Stream data. */
		TArray<uint8> Data;

		/** Current buffer position. */
		int32 Position = 0;

		/** Whether stream is used to read (true) or write (false) */
		bool bIsReadStream = false;
	};
} // namespace UE::OSC

class OSC_API FOSCStream : public UE::OSC::FStream
{
public:
	UE_DEPRECATED(5.5, "Use UE::OSC::FStream instead")
	FOSCStream() = default;

	UE_DEPRECATED(5.5, "Use UE::OSC::FStream instead")
	FOSCStream(const uint8* InData, int32 InSize) : UE::OSC::FStream(InData, InSize) { }
};
