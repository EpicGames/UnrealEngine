// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"
#include "Containers/StringView.h"
#include "Math/Color.h"
#include "Misc/TVariant.h"


namespace UE::OSC
{
	enum class EDataType : uint8
	{
		Blob = 'b',
		Char = 'c',
		Color = 'r',
		Double = 'd',
		False = 'F',
		Float = 'f',
		Infinitum = 'I',
		Int32 = 'i',
		Int64 = 'h',
		NilValue = 'N',
		String = 's',
		Terminate = '\0',
		Time = 't',
		True = 'T'
	};

	const OSC_API TCHAR* LexToString(EDataType DataType);

	class OSC_API FOSCData
	{
	public:
		FOSCData() = default;

		explicit FOSCData(const TArray<uint8>& Value);
		explicit FOSCData(TArray<uint8>&& Value);
		explicit FOSCData(bool Value);
		explicit FOSCData(ANSICHAR Value);
		explicit FOSCData(FColor Value);
		explicit FOSCData(double Value);
		explicit FOSCData(float Value);
		explicit FOSCData(int32 Value);
		explicit FOSCData(int64 Value);
		explicit FOSCData(FString Value);
		explicit FOSCData(uint64 Value);

		UE_DEPRECATED(5.5, "Use applicable explicitly typed constructor or static construction function")
		explicit FOSCData(EDataType DataType);

		static const FOSCData& NilData();
		static const FOSCData& Infinitum();
		static const FOSCData& Terminate();
		static bool IsNil(const FOSCData& InType);

		EDataType GetDataType() const;

		inline bool IsBlob() const { return DataType == EDataType::Blob; }
		inline bool IsBool() const { return DataType == EDataType::True || DataType == EDataType::False; }
		inline bool IsChar() const { return DataType == EDataType::Char; }
		inline bool IsColor() const { return DataType == EDataType::Color; }
		inline bool IsDouble() const { return DataType == EDataType::Double; }
		inline bool IsFloat() const { return DataType == EDataType::Float; }
		inline bool IsInfinitum() const { return DataType == EDataType::Infinitum; }
		inline bool IsInt32() const { return DataType == EDataType::Int32; }
		inline bool IsInt64() const { return DataType == EDataType::Int64; }
		inline bool IsNil() const { return DataType == EDataType::NilValue; }
		inline bool IsString() const { return DataType == EDataType::String; }
		inline bool IsTimeTag() const { return DataType == EDataType::Time; }
		inline bool IsTerminate() const { return DataType == EDataType::Terminate; }

		TArray<uint8> GetBlob() const;
		bool GetBool() const;
		ANSICHAR GetChar() const;
		FColor GetColor() const;
		double GetDouble() const;
		float GetFloat() const;
		int32 GetInt32() const;
		int64 GetInt64() const;
		FString GetString() const;
		uint64 GetTimeTag() const;

		TArrayView<const uint8> GetBlobArrayView() const;
		FStringView GetStringView() const;

		using FVariant = TVariant
		<
			TArray<uint8>,	// Blob
			bool,
			ANSICHAR,
			FColor,
			double,
			float,
			int32,
			int64,
			FString,
			uint64			// TimeTag
		>;

	protected:
		EDataType DataType = EDataType::NilValue;
		FVariant Data;
	};
} // namespace UE::OSC


// Exists for back compat.  To be deprecated
enum EOSCTypeTag
{
	OSC_BLOB = 'b',
	OSC_CHAR = 'c',
	OSC_COLOR = 'r',
	OSC_DOUBLE = 'd',
	OSC_FALSE = 'F',
	OSC_FLOAT = 'f',
	OSC_INFINITUM = 'I',
	OSC_INT32 = 'i',
	OSC_INT64 = 'h',
	OSC_NIL = 'N',
	OSC_STRING = 's',
	OSC_TERMINATE = '\0',
	OSC_TIME = 't',
	OSC_TRUE = 'T'
};

class OSC_API FOSCType : public UE::OSC::FOSCData
{
public:
	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(const TArray<uint8>& Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(TArray<uint8>&& Value)
		: UE::OSC::FOSCData(MoveTemp(Value))
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(bool Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(ANSICHAR Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(FColor Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(double Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(float Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(int32 Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(int64 Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(const FString& Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(FString&& Value)
		: UE::OSC::FOSCData(MoveTemp(Value))
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData instead")
	explicit FOSCType(uint64 Value)
		: UE::OSC::FOSCData(Value)
	{
	}

	UE_DEPRECATED(5.5, "Use UE::OSC::FOSCData::GetDataType() instead")
	int32 GetTypeTag() const { return static_cast<int32>(UE::OSC::EDataType::NilValue); }
};
